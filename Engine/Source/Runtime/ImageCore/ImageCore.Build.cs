// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ImageCore : ModuleRules
{
	public ImageCore(ReadOnlyTargetRules Target) : base(Target)
	{
		// include only, no link :
		// ImageCoreUtils.h includes TextureDefines.h from Engine, so technically it should be a "PublicIncludePathModuleNames" include,
		// but that breaks the build because there are other modules that use ImageCore that have conflicts with Engine headers.
		// So, leave it as PrivateIncludePathModuleNames, but then any module that uses ImageCoreUtils needs to PrivateIncludePathModuleNames Engine.
		// PublicIncludePathModuleNames
		PrivateIncludePathModuleNames.AddRange(new string[]
		{
			"Engine",
		});

		PrivateDependencyModuleNames.Add("CoreUObject"); // for TextureDefines.h
		PrivateDependencyModuleNames.Add("stb_image_resize2");

		PublicDependencyModuleNames.Add("Core");
	}
}
