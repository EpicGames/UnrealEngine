// Copyright Epic Games, Inc. All Rights Reserved.

#include "DDSFile.h"
#include "Logging/LogMacros.h"

DEFINE_LOG_CATEGORY_STATIC(LogDDSFile, Log, All);

namespace UE { namespace DDS
{

constexpr uint32 MakeFOURCC(uint32 a, uint32 b, uint32 c, uint32 d) { return ((a) | ((b) << 8) | ((c) << 16) | ((d) << 24)); }

constexpr uint32 DDSD_CAPS = 0x00000001;
constexpr uint32 DDSD_HEIGHT = 0x00000002;
constexpr uint32 DDSD_WIDTH = 0x00000004;
constexpr uint32 DDSD_PITCH = 0x00000008;
constexpr uint32 DDSD_PIXELFORMAT = 0x00001000;
constexpr uint32 DDSD_MIPMAPCOUNT = 0x00020000;
constexpr uint32 DDSD_DEPTH = 0x00800000;

constexpr uint32 DDPF_ALPHA = 0x00000002;
constexpr uint32 DDPF_FOURCC = 0x00000004;
constexpr uint32 DDPF_RGB = 0x00000040;
constexpr uint32 DDPF_LUMINANCE = 0x00020000;
constexpr uint32 DDPF_BUMPDUDV = 0x00080000;

constexpr uint32 DDSCAPS_COMPLEX = 0x00000008;
constexpr uint32 DDSCAPS_TEXTURE = 0x00001000;
constexpr uint32 DDSCAPS_MIPMAP = 0x00400000;

constexpr uint32 DDSCAPS2_CUBEMAP = 0x00000200;
constexpr uint32 DDSCAPS2_VOLUME = 0x00200000;

constexpr uint32 RESOURCE_DIMENSION_UNKNOWN = 0;
constexpr uint32 RESOURCE_DIMENSION_BUFFER = 1;
constexpr uint32 RESOURCE_DIMENSION_TEXTURE1D = 2;
constexpr uint32 RESOURCE_DIMENSION_TEXTURE2D = 3;
constexpr uint32 RESOURCE_DIMENSION_TEXTURE3D = 4;

constexpr uint32 RESOURCE_MISC_TEXTURECUBE = 0x00000004;

constexpr uint32 DDS_MAGIC = MakeFOURCC('D', 'D', 'S', ' ');
constexpr uint32 DX10_MAGIC = MakeFOURCC('D', 'X', '1', '0');

struct FDDSPixelFormat
{
	uint32 size;
	uint32 flags;
	uint32 fourCC;
	uint32 RGBBitCount;
	uint32 RBitMask;
	uint32 GBitMask;
	uint32 BBitMask;
	uint32 ABitMask;
};

struct FDDSHeaderWithMagic 
{
	uint32 Magic; // Must be DDS_MAGIC
	uint32 size;
	uint32 flags;
	uint32 height;
	uint32 width;
	uint32 pitchOrLinearSize;
	uint32 depth;	 
	uint32 num_mips;
	uint32 reserved1[11];
	FDDSPixelFormat ddspf;
	uint32 caps;
	uint32 caps2;
	uint32 caps3;
	uint32 caps4;
	uint32 reserved2;
};

struct FDDSHeaderDX10 
{
	uint32 dxgi_format;
	uint32 resource_dimension;
	uint32 misc_flag;	 // see D3D11_RESOURCE_MISC_FLAG
	uint32 array_size;
	uint32 misc_flag2;
};

struct FDXGIFormatName 
{
	EDXGIFormat Format;
	const TCHAR* Name;
};

const TCHAR * DXGIFormatGetName(EDXGIFormat fmt) 
{
	static const FDXGIFormatName FormatList[] = 
	{
#define RGBFMT(name,id,bypu) { EDXGIFormat::name, TEXT(#name) },
#define BCNFMT(name,id,bypu) { EDXGIFormat::name, TEXT(#name) },
#define ODDFMT(name,id) { EDXGIFormat::name, TEXT(#name) },
		UE_DXGI_FORMAT_LIST
#undef RGBFMT
#undef BCNFMT
#undef ODDFMT
	};

	for(size_t i = 0; i < sizeof(FormatList) / sizeof(*FormatList); ++i) 
	{
		if (FormatList[i].Format == fmt) 
		{
			return FormatList[i].Name;
		}
	}
	return FormatList[0].Name; // first entry is "unknown format"
}

// list of non-sRGB / sRGB pixel format pairs: even=UNORM, odd=UNORM_SRGB
// (sorted by DXGI_FORMAT code)
static const EDXGIFormat DXGIFormatSRGBTable[] = 
{
	EDXGIFormat::R8G8B8A8_UNORM,		EDXGIFormat::R8G8B8A8_UNORM_SRGB,
	EDXGIFormat::BC1_UNORM,				EDXGIFormat::BC1_UNORM_SRGB,
	EDXGIFormat::BC2_UNORM,				EDXGIFormat::BC2_UNORM_SRGB,
	EDXGIFormat::BC3_UNORM,				EDXGIFormat::BC3_UNORM_SRGB,
	EDXGIFormat::B8G8R8A8_UNORM,		EDXGIFormat::B8G8R8A8_UNORM_SRGB,
	EDXGIFormat::B8G8R8X8_UNORM,		EDXGIFormat::B8G8R8X8_UNORM_SRGB,
	EDXGIFormat::BC7_UNORM,				EDXGIFormat::BC7_UNORM_SRGB,
};

// List of corresponding RGBA/BGRA format pairs: even=RGBA, odd=BGRA
static const EDXGIFormat DXGIFormatRGBATable[] =
{
	EDXGIFormat::R8G8B8A8_TYPELESS,		EDXGIFormat::B8G8R8A8_TYPELESS,
	EDXGIFormat::R8G8B8A8_UNORM,		EDXGIFormat::B8G8R8A8_UNORM,
	EDXGIFormat::R8G8B8A8_UNORM_SRGB,	EDXGIFormat::B8G8R8A8_UNORM_SRGB,
	EDXGIFormat::R8G8B8X8,				EDXGIFormat::B8G8R8X8_TYPELESS,
// not mapped :
//	RGBFMT(R8G8B8A8_UINT,				30, 4) 
//	RGBFMT(R8G8B8A8_SNORM,				31, 4) 
//	RGBFMT(R8G8B8A8_SINT,				32, 4) 
};

static int DXGIFormatFindInTableImpl(const EDXGIFormat* InTable, int InCount, EDXGIFormat InFormat)
{
	for (int i = 0; i < InCount; ++i)
	{
		if (InTable[i] == InFormat)
		{
			return i;
		}
	}

	return -1;
}

template<int N>
static int DXGIFormatFindInTable(const EDXGIFormat (&InTable)[N], EDXGIFormat InFormat)
{
	return DXGIFormatFindInTableImpl(InTable, N, InFormat);
}

bool DXGIFormatIsSRGB(EDXGIFormat Format) 
{
	int idx = DXGIFormatFindInTable(DXGIFormatSRGBTable, Format);
	return idx >= 0 && ((idx & 1) == 1);
}

bool DXGIFormatHasLinearAndSRGBForm(EDXGIFormat Format) 
{
	int idx = DXGIFormatFindInTable(DXGIFormatSRGBTable, Format);
	return idx >= 0;
}

EDXGIFormat DXGIFormatRemoveSRGB(EDXGIFormat fmt) 
{
	int idx = DXGIFormatFindInTable(DXGIFormatSRGBTable, fmt);
	if(idx >= 0)
	{
		return DXGIFormatSRGBTable[idx & ~1];
	}
	else 
	{
		return fmt;
	}
}

EDXGIFormat DXGIFormatAddSRGB(EDXGIFormat fmt) 
{
	int idx = DXGIFormatFindInTable(DXGIFormatSRGBTable, fmt);
	if(idx >= 0) 
	{
		return DXGIFormatSRGBTable[idx | 1];
	}
	else 
	{
		return fmt;
	}
}

// this is used for trying to map old format specifications to DXGI.
struct FBitmaskToDXGI 
{
	uint32 Flags;
	uint32 Bits;
	uint32 RMask, GMask, BMask, AMask;
	EDXGIFormat Format;
	int32 AutoD3d9;
};

// used for mapping fourcc format specifications to DXGI
struct FFOURCCToDXGI 
{
	uint32 fourcc;
	EDXGIFormat Format;
	int32 AutoD3d9;
};

struct FDXGIFormatInfo 
{
	EDXGIFormat Format;
	uint32 UnitWidth; // width of a coding unit
	uint32 UnitHeight; // height of a coding unit
	uint32 UnitBytes;
};

static const FDXGIFormatInfo SupportedFormatList[] = 
{
#define RGBFMT(name,id,bypu) { EDXGIFormat::name, 1,1, bypu },
#define BCNFMT(name,id,bypu) { EDXGIFormat::name, 4,4, bypu },
#define ODDFMT(name,id) // these are not supported for reading so they're intentionally not on the list
	UE_DXGI_FORMAT_LIST
#undef RGBFMT
#undef BCNFMT
#undef ODDFMT
};

// This is following MS DDSTextureLoader11.
//
// Formats with AutoD3d9 == 1 will get emitted as D3D9 DDS when FormatVersion is "Auto";
// the other ones we read from D3D9 .DDS files but will only write when FormatVersion is D3D9,
// and otherwise prefer D3D10 mode, because they're not widely supported in apps that consume
// D3D9 .DDS files.
//
static const FBitmaskToDXGI BitmaskToDXGITable[] = 
{
	//flags				bits	r			g			b			a			dxgi							AutoD3d9
	{ DDPF_RGB,			32,		0x000000ff, 0x0000ff00, 0x00ff0000, 0xff000000, EDXGIFormat::R8G8B8A8_UNORM,	1 },
	{ DDPF_RGB,			32,		0x00ff0000, 0x0000ff00, 0x000000ff, 0xff000000, EDXGIFormat::B8G8R8A8_UNORM,	1 },
	{ DDPF_RGB,			32,		0x00ff0000, 0x0000ff00, 0x000000ff, 0x00000000, EDXGIFormat::B8G8R8X8_UNORM,	1 },
	{ DDPF_RGB,			32,		0x3ff00000, 0x000ffc00, 0x000003ff, 0xc0000000, EDXGIFormat::R10G10B10A2_UNORM,	0 }, // This mask is backwards, but that's the standard value to write for R10G10B10A2_UNORM! (see comments in DDSTextureLoader11)
	{ DDPF_RGB,			32,		0x0000ffff, 0xffff0000, 0x00000000, 0x00000000, EDXGIFormat::R16G16_UNORM,		1 },
	{ DDPF_RGB,			32,		0xffffffff, 0x00000000, 0x00000000, 0x00000000, EDXGIFormat::R32_FLOAT,			0 }, // only 32-bit color channel format in D3D9 was R32F
	{ DDPF_RGB,			16,		0x7c00,		0x03e0,		0x001f,		0x8000,		EDXGIFormat::B5G5R5A1_UNORM,	1 },
	{ DDPF_RGB,			16,		0xf800,		0x07e0,		0x001f,		0x0000,		EDXGIFormat::B5G6R5_UNORM,		1 },
	{ DDPF_RGB,			16,		0x0f00,		0x00f0,		0x000f,		0xf000,		EDXGIFormat::B4G4R4A4_UNORM,	1 },
	{ DDPF_LUMINANCE,	8,		0xff,		0x00,		0x00,		0x00,		EDXGIFormat::R8_UNORM,			1 },
	{ DDPF_LUMINANCE,	16,		0xffff,		0x0000,		0x0000,		0x0000,		EDXGIFormat::R16_UNORM,			1 },
	{ DDPF_LUMINANCE,	16,		0x00ff,		0x0000,		0x0000,		0xff00,		EDXGIFormat::R8G8_UNORM,		1 }, // official way to do it - this must go first!
	{ DDPF_LUMINANCE,	8,		0xff,		0x00,		0x00,		0xff00,		EDXGIFormat::R8G8_UNORM,		0 }, // some writers write this instead, ugh.
	{ DDPF_ALPHA,		8,		0x00,		0x00,		0x00,		0xff,		EDXGIFormat::A8_UNORM,			1 },
	{ DDPF_BUMPDUDV,	32,		0x000000ff, 0x0000ff00, 0x00ff0000, 0xff000000, EDXGIFormat::R8G8B8A8_SNORM,	0 },
	{ DDPF_BUMPDUDV,	2,		0x0000ffff, 0xffff0000, 0x00000000, 0x00000000, EDXGIFormat::R16G16_SNORM,		0 },
	{ DDPF_BUMPDUDV,	16,		0x00ff,		0xff00,		0x0000,		0x0000,		EDXGIFormat::R8G8_SNORM,		0 },
	{ DDPF_RGB,			32,		0x000000ff, 0x0000ff00, 0x00ff0000, 0x00000000, EDXGIFormat::R8G8B8X8,	1 },
	{ DDPF_RGB,			24,		0x000000ff, 0x0000ff00, 0x00ff0000, 0x00000000, EDXGIFormat::R8G8B8,	1 },
	{ DDPF_RGB,			24,		0x00ff0000, 0x0000ff00, 0x000000ff, 0x00000000, EDXGIFormat::B8G8R8,	1 },
};



// This is following MS DDSTextureLoader11.
// When multiple FOURCCs map to the same DXGI format, put the preferred FOURCC first.
// AutoD3d9 works as above.
static const FFOURCCToDXGI FOURCCToDXGITable[] = 
{
	//fourcc							dxgi								AutoD3d9
	{ MakeFOURCC('D','X','T','1'),		EDXGIFormat::BC1_UNORM,				1 },
	{ MakeFOURCC('D','X','T','3'),		EDXGIFormat::BC2_UNORM,				1 }, // Note: comes before DXT2 because it's our preferred choice for BC2 export
	{ MakeFOURCC('D','X','T','2'),		EDXGIFormat::BC2_UNORM,				1 },
	{ MakeFOURCC('D','X','T','5'),		EDXGIFormat::BC3_UNORM,				1 }, // Note: comes before DXT4 because it's our preferred choice for BC3 export
	{ MakeFOURCC('D','X','T','4'),		EDXGIFormat::BC3_UNORM,				1 },
	{ MakeFOURCC('B','C','4','U'),		EDXGIFormat::BC4_UNORM,				0 },
	{ MakeFOURCC('B','C','4','S'),		EDXGIFormat::BC4_SNORM,				0 },
	{ MakeFOURCC('A','T','I','1'),		EDXGIFormat::BC4_UNORM,				0 }, // Note: prefer the more explicit BC4U to ATI1 for export
	{ MakeFOURCC('B','C','5','U'),		EDXGIFormat::BC5_UNORM,				0 },
	{ MakeFOURCC('B','C','5','S'),		EDXGIFormat::BC5_SNORM,				0 },
	{ MakeFOURCC('A','T','I','2'),		EDXGIFormat::BC5_UNORM,				0 }, // NOTE: ATI2 is kind of odd (technically swapped block order), so put it below BC5U
	{ MakeFOURCC('B','C','6','H'),		EDXGIFormat::BC6H_UF16,				0 },
	{ MakeFOURCC('B','C','7','L'),		EDXGIFormat::BC7_UNORM,				0 },
	{ MakeFOURCC('B','C','7', 0 ),		EDXGIFormat::BC7_UNORM,				0 },
	{ 36,								EDXGIFormat::R16G16B16A16_UNORM,	1 }, // D3DFMT_A16B16G16R16
	{ 110,								EDXGIFormat::R16G16B16A16_SNORM,	0 }, // D3DFMT_Q16W16V16U16
	{ 111,								EDXGIFormat::R16_FLOAT,				1 }, // D3DFMT_R16F
	{ 112,								EDXGIFormat::R16G16_FLOAT,			1 }, // D3DFMT_G16R16F
	{ 113,								EDXGIFormat::R16G16B16A16_FLOAT,	1 }, // D3DFMT_A16B16G16R16F
	{ 114,								EDXGIFormat::R32_FLOAT,				1 }, // D3DFMT_R32F
	{ 115,								EDXGIFormat::R32G32_FLOAT,			1 }, // D3DFMT_G32R32F
	{ 116,								EDXGIFormat::R32G32B32A32_FLOAT,	1 }, // D3DFMT_A32B32G32R32F
};

static const FDXGIFormatInfo* DXGIFormatGetInfo(EDXGIFormat InFormat)
{
	// need to handle this special because UNKNOWN _does_ appear in the SupportedFormatList
	// but we don't want to treat it as legal
	if (InFormat == EDXGIFormat::UNKNOWN)
	{
		return 0;
	}

	for (size_t i = 0; i < sizeof(SupportedFormatList)/sizeof(*SupportedFormatList); ++i)
	{
		if (InFormat == SupportedFormatList[i].Format) 
		{
			return &SupportedFormatList[i];
		}
	}

	return 0;
}

static EDXGIFormat DXGIFormatFromDDS9Header(const FDDSHeaderWithMagic* InDDSHeader)
{
	// The old format can be specified either with FOURCC or with some bit masks, so we use
	// this to determine the corresponding dxgi format.
	const FDDSPixelFormat &ddpf = InDDSHeader->ddspf;
	if (ddpf.flags & DDPF_FOURCC) 
	{
		for (size_t i = 0; i < sizeof(FOURCCToDXGITable)/sizeof(*FOURCCToDXGITable); ++i) 
		{
			if (ddpf.fourCC == FOURCCToDXGITable[i].fourcc) 
			{
				return FOURCCToDXGITable[i].Format;
			}
		}
	}
	else 
	{
		uint32 type_flags = ddpf.flags & (DDPF_RGB | DDPF_LUMINANCE | DDPF_ALPHA | DDPF_BUMPDUDV);
		for (size_t i = 0; i < sizeof(BitmaskToDXGITable)/sizeof(*BitmaskToDXGITable); ++i) 
		{
			const FBitmaskToDXGI *fmt = &BitmaskToDXGITable[i];
			if (type_flags == fmt->Flags && ddpf.RGBBitCount == fmt->Bits &&
				ddpf.RBitMask == fmt->RMask && ddpf.GBitMask == fmt->GMask &&
				ddpf.BBitMask == fmt->BMask && ddpf.ABitMask == fmt->AMask)
			{
				return fmt->Format;
			}
		}
	}

	return EDXGIFormat::UNKNOWN;
}

static uint32 MipDimension(uint32 dim, uint32 level) 
{
	check( level <= FDDSFile::MAX_MIPS_SUPPORTED ); // guaranteed less than 32

	// mip dimensions truncate at every level and bottom out at 1
	uint32 x = dim >> level;
	return x ? x : 1;
}

static void InitMip(FDDSMip* InMip, uint32 InWidth, uint32 InHeight, uint32 InDepth, const FDXGIFormatInfo* InFormatInfo) 
{
	uint32 width_u = (InWidth + InFormatInfo->UnitWidth-1) / InFormatInfo->UnitWidth;
	uint32 height_u = (InHeight + InFormatInfo->UnitHeight-1) / InFormatInfo->UnitHeight;

	InMip->Width = InWidth;
	InMip->Height = InHeight;
	InMip->Depth = InDepth;
	InMip->RowStride = (int64)width_u * InFormatInfo->UnitBytes;
	InMip->SliceStride = (int64)height_u * InMip->RowStride;
	InMip->DataSize = (int64)InDepth * InMip->SliceStride;
	InMip->Data = 0;
}

static int64 InitMips(FDDSFile* InDDS, const FDXGIFormatInfo* InFormatInfo)
{
	int64 AllMipsSize = 0;
	for (uint32 ArrayIndex = 0; ArrayIndex < InDDS->ArraySize; ++ArrayIndex)
	{
		for (uint32 MipIndex = 0; MipIndex < InDDS->MipCount; ++MipIndex)
		{
			FDDSMip* Mip = &InDDS->Mips[ArrayIndex * InDDS->MipCount + MipIndex];
			uint32 MipWidth = MipDimension(InDDS->Width, MipIndex);
			uint32 MipHeight = MipDimension(InDDS->Height, MipIndex);
			uint32 MipDepth = MipDimension(InDDS->Depth, MipIndex);
			InitMip(Mip, MipWidth, MipHeight, MipDepth, InFormatInfo);
			AllMipsSize += Mip->DataSize;
		}
	}

	return AllMipsSize;
}

EDDSError FDDSFile::Validate() const
{
	// Supported pixel format?
	const FDXGIFormatInfo* FormatInfo = DXGIFormatGetInfo(DXGIFormat);
	if (!FormatInfo)
	{
		UE_LOGF(LogDDSFile, Warning, "Unsupported format %d (%ls)", int(DXGIFormat), DXGIFormatGetName(DXGIFormat));
		return EDDSError::BadPixelFormat;
	}

	// Resource and image dimensions agree?
	switch (Dimension)
	{
	case 1:
		if (Height != 1 || Depth != 1)
		{
			UE_LOGF(LogDDSFile, Warning, "1D textures must have height and depth of 1.");
			return EDDSError::BadImageDimension;
		}
		break;

	case 2:
		if (Depth != 1)
		{
			UE_LOGF(LogDDSFile, Warning, "2D textures must have depth of 1.");
			return EDDSError::BadImageDimension;
		}
		break;

	case 3:
		if (ArraySize != 1)
		{
			UE_LOGF(LogDDSFile, Warning, "3D textures must have array size of 1.");
			return EDDSError::BadImageDimension;
		}
		break;

	default:
		UE_LOGF(LogDDSFile, Warning, "DDS textures must be 1D, 2D or 3D.");
		return EDDSError::BadResourceDimension;
	}

	// All dimensions must be non-zero
	if (Width == 0 || Height == 0 || Depth == 0 || MipCount == 0 || ArraySize == 0)
	{
		UE_LOGF(LogDDSFile, Warning, "One or more image dimensions are zero.");
		return EDDSError::BadImageDimension;
	}

	// Images must not be larger than we support
	check( MAX_MIPS_SUPPORTED < 32 );
	const uint32 MaxDimension = (1 << MAX_MIPS_SUPPORTED) - 1; // 1<<k is when we tip over into needing k+1 mip levels, (1<<k)-1 needs just k.
	if (Width > MaxDimension || Height > MaxDimension || Depth > MaxDimension)
	{
		UE_LOGF(LogDDSFile, Warning, "Image dimensions %ux%ux%u of DDS exceed maximum of %u.", Width, Height, Depth, MaxDimension);
		return EDDSError::BadImageDimension;
	}

	// Check that mipmap count is supported and makes sense for the image dimensions.
	if (MipCount > MAX_MIPS_SUPPORTED)
	{
		UE_LOGF(LogDDSFile, Warning, "Invalid mipmap count of %u.", MipCount);
		return EDDSError::BadMipmapCount;
	}
	
	// ArraySize * MipCount must fit in 32 bits
	// checking against max dim will ensure that :
	if ( ArraySize > MaxDimension )
	{
		UE_LOGF(LogDDSFile, Warning, "ArraySize %ux of DDS exceed maximum of %u.", ArraySize, MaxDimension);
		return EDDSError::BadImageDimension;		
	}

	// all dimensions are limited to 20 bits, so you could get to 1<<60 pixel count
	//	with 16-byte pixels the bytes per surface could then go to negative in int64
	uint64 MaxPixelCount = 1ULL<<40; // could be higher and not overflow int64 , but would run out of memory anyway
	// note the limit at MaxPixelCount is not strict due to use of floats, but doesn't need to be
	if ( float(Width) * float(Height) * float(Depth) * float(ArraySize) > float(MaxPixelCount) )
	{
		UE_LOGF(LogDDSFile, Warning, "Image dimensions %ux%ux%ux%u of DDS exceed maximum pixel count.", Width, Height, Depth, ArraySize);
		return EDDSError::BadImageDimension;
	}

	// Mipmaps halve each dimension (rounding down) every step; dimensions that end up at 0
	// turn into 1. For the mip count to be valid, in the final mip level, at least one
	// dimension should have been 0 before this adjustment.
	const uint32 FinalMip = MipCount - 1;
	if ((Width >> FinalMip) == 0 &&
		(Height >> FinalMip) == 0 &&
		(Depth >> FinalMip) == 0)
	{
		UE_LOGF(LogDDSFile, Warning, "Invalid mipmap count of %u for %ux%ux%u image.", MipCount, Width, Height, Depth);
		return EDDSError::BadMipmapCount;
	}

	// Cubemaps need to be square and have a valid array count
	if (CreateFlags & CREATE_FLAG_CUBEMAP)
	{
		if (Dimension != 2 )
		{
			UE_LOGF(LogDDSFile, Warning, "Cubemap must be dimension 2!");
			return EDDSError::BadCubemap;
		}

		if (Width != Height || Depth != 1)
		{
			UE_LOGF(LogDDSFile, Warning, "Cubemap has non-square faces or non-1 depth!");
			return EDDSError::BadCubemap;
		}

		if (ArraySize < 6 || (ArraySize % 6) != 0)
		{
			UE_LOGF(LogDDSFile, Warning, "Cubemap or cubemap array doesn't have a multiple of 6 faces.");
			return EDDSError::BadCubemap;
		}
	}

	return EDDSError::OK;
}

bool FDDSFile::IsValidTexture2D() const
{
	if ( Validate() != EDDSError::OK )
	{
		return false;
	}

	if ( Dimension == 1 || Dimension == 2 )
	{
		if ( ArraySize == 1 )
		{
			return true;
		}
	}

	return false;
}

bool FDDSFile::IsValidTextureCube() const
{
	if ( Validate() != EDDSError::OK )
	{
		return false;
	}
	
	if (CreateFlags & CREATE_FLAG_CUBEMAP)
	{
		return true;
	}

	return false;
}

bool FDDSFile::IsValidTextureArray() const
{
	if ( Validate() != EDDSError::OK )
	{
		return false;
	}

	if (CreateFlags & CREATE_FLAG_CUBEMAP)
	{
		// don't identify cubes as arrays
		return false;
	}

	if ( Dimension == 1 || Dimension == 2 )
	{
		if ( ArraySize > 1 )
		{
			return true;
		}
	}

	return false;
}

bool FDDSFile::IsValidTextureVolume() const
{
	if ( Validate() != EDDSError::OK )
	{
		return false;
	}
	
	if ( Dimension == 3 )
	{
		// checked in Validate :
		check( ArraySize == 1 );
		return true;
	}

	return false;
}

static EDDSError AllocateMips(FDDSFile* InDDS, const FDXGIFormatInfo* InFormatInfo, uint32 InCreateFlags)
{
	InDDS->Mips.Empty();
	InDDS->Mips.SetNumZeroed(InDDS->ArraySize * InDDS->MipCount);

	if (!(InCreateFlags & FDDSFile::CREATE_FLAG_NO_MIP_STORAGE_ALLOC)) 
	{
		// Allocate storage for all the mip levels

		// first pass, add up all sizes
		//	then alloc it, second pass hand out all the pointers

		int64 AllMipsSize = InitMips(InDDS, InFormatInfo);

		InDDS->MipRawData.SetNumUninitialized(AllMipsSize);

		int64 CurrentOffset = 0;
		for (auto& Mip : InDDS->Mips)
		{
			Mip.Data = &InDDS->MipRawData[CurrentOffset];
			CurrentOffset += Mip.DataSize;
		}

		check(CurrentOffset == AllMipsSize);
	}

	return EDDSError::OK;
}

/* static */ FDDSFile* FDDSFile::CreateEmpty(int InDimension, uint32 InWidth, uint32 InHeight, uint32 InDepth, uint32 InMipCount, uint32 InArraySize, EDXGIFormat InFormat, uint32 InCreateFlags, EDDSError* OutError)
{
	// If null OutError passed, point to somewhere safe
	EDDSError DummyError;
	if (!OutError)
		OutError = &DummyError;

	// Allocate the new DDS
	FDDSFile* DDS = new FDDSFile();
	if (!DDS)
	{
		*OutError = EDDSError::OutOfMemory;
		return nullptr;
	}

	// Set up the parameters
	DDS->Dimension = InDimension;
	DDS->Width = InWidth;
	DDS->Height = InHeight;
	DDS->Depth = InDepth;
	DDS->MipCount = InMipCount;
	DDS->ArraySize = InArraySize;
	DDS->DXGIFormat = InFormat;
	DDS->CreateFlags = InCreateFlags & ~CREATE_FLAG_NO_MIP_STORAGE_ALLOC;

	// Check all the parameters
	*OutError = DDS->Validate();
	if (*OutError != EDDSError::OK)
	{
		delete DDS;
		return nullptr;
	}

	// Try to allocate mip storage
	// Validate checks the pixel format is OK, so we don't need to re-check here
	const FDXGIFormatInfo* FormatInfo = DXGIFormatGetInfo(InFormat);
	*OutError = AllocateMips(DDS, FormatInfo, InCreateFlags);
	if (*OutError != EDDSError::OK) //-V547
	{
		delete DDS;
		return nullptr;
	}

	return DDS;
}

/* static */ FDDSFile* FDDSFile::CreateEmpty2D(uint32 InWidth, uint32 InHeight, uint32 InMipCount, EDXGIFormat InFormat, uint32 InCreateFlags, EDDSError* OutError)
{
	return FDDSFile::CreateEmpty(2, InWidth, InHeight, 1, InMipCount, 1, InFormat, InCreateFlags, OutError);
}

static EDDSError ParseHeader(FDDSFile* InDDS, FDDSHeaderWithMagic const* InHeader, FDDSHeaderDX10 const* InDX10Header)
{
	// If the fourCC is "DX10" then we have a secondary header that follows the first header. 
	// This header specifies an dxgi_format explicitly, so we don't have to derive one.
	bool bDX10 = false;
	const FDDSPixelFormat& ddpf = InHeader->ddspf;
	if ((ddpf.flags & DDPF_FOURCC) && ddpf.fourCC == DX10_MAGIC)
	{
		if (InDX10Header->resource_dimension >= RESOURCE_DIMENSION_TEXTURE1D && InDX10Header->resource_dimension <= RESOURCE_DIMENSION_TEXTURE3D)
		{
			InDDS->Dimension = (InDX10Header->resource_dimension - RESOURCE_DIMENSION_TEXTURE1D) + 1;
		}
		else
		{
			UE_LOGF(LogDDSFile, Warning, "D3D10 resource dimension in DDS is neither 1D, 2D, nor 3D.");
			return EDDSError::BadResourceDimension;
		}
		InDDS->DXGIFormat = (EDXGIFormat)InDX10Header->dxgi_format;
		bDX10 = true;
	}
	else
	{
		// For D3D9-style files, we guess dimension from the caps bits.
		// If the volume cap is set, assume 3D, otherwise 2D.
		InDDS->Dimension = (InHeader->caps2 & DDSCAPS2_VOLUME) ? 3 : 2;
		InDDS->DXGIFormat = DXGIFormatFromDDS9Header(InHeader);
	}

	// More header parsing
	bool bIsCubemap = bDX10 ? (InDX10Header->misc_flag & RESOURCE_MISC_TEXTURECUBE) != 0 : (InHeader->caps2 & DDSCAPS2_CUBEMAP) != 0;
	bool bIsVolume = InDDS->Dimension == 3;

	InDDS->Width = InHeader->width;
	InDDS->Height = InHeader->height;
	InDDS->Depth = bIsVolume ? InHeader->depth : 1;
	InDDS->MipCount = (InHeader->caps & DDSCAPS_MIPMAP) ? InHeader->num_mips : 1;
	InDDS->ArraySize = bDX10 ? InDX10Header->array_size : 1;
	InDDS->CreateFlags = 0;
	if (bIsCubemap)
	{
		InDDS->CreateFlags |= FDDSFile::CREATE_FLAG_CUBEMAP;
		InDDS->ArraySize *= 6;
	}

	if (!bDX10)
	{
		InDDS->CreateFlags |= FDDSFile::CREATE_FLAG_WAS_D3D9;
	}

	// Sanity-check header values and return
	EDDSError Error = InDDS->Validate();
	
	return Error;
}

static EDDSError ReadPayload(FDDSFile* InDDS, const uint8* InReadCursor, const uint8* InReadEnd)
{
	const FDXGIFormatInfo* FormatInfo = DXGIFormatGetInfo(InDDS->DXGIFormat);
	EDDSError Error = AllocateMips(InDDS, FormatInfo, InDDS->CreateFlags);
	if (Error != EDDSError::OK)
	{
		return Error;
	}

	for (auto& Mip : InDDS->Mips)
	{
		if (InReadEnd - InReadCursor < Mip.DataSize)
		{
			UE_LOGF(LogDDSFile, Warning, "Error reading texture data.");
			return EDDSError::IoError;
		}

		memcpy(Mip.Data, InReadCursor, Mip.DataSize);
		InReadCursor += Mip.DataSize;
	}

	return EDDSError::OK;
}


int64 GetDDSHeaderMaximalSize()
{
	return sizeof(FDDSHeaderWithMagic) + sizeof(FDDSHeaderDX10);
}

int64 GetDDSHeaderMinimalSize()
{
	return sizeof(FDDSHeaderWithMagic);
}


FDDSFile* FDDSFile::CreateFromDDSInMemory(const uint8* InDDS, int64 InDDSSize, EDDSError* OutError, bool bHeaderOnly)
{
	return CreateFromDDSInMemory(InDDS, InDDSSize, OutError, static_cast<EDDSReadMipMode>(bHeaderOnly));
}

/* static */ FDDSFile* FDDSFile::CreateFromDDSInMemory(const uint8* InDDS, int64 InDDSSize, EDDSError *OutError, EDDSReadMipMode InMipReadMode)
{
	// If no OutError passed in, redirect it to dummy storage on stack.
	EDDSError DummyError;
	if (!OutError)
	{
		OutError = &DummyError;
	}

	FDDSHeaderWithMagic DDSHeader = {};
	FDDSHeaderDX10 DDS10Header = {};

	// If we don't even have enough bytes for this to contain a valid header,
	// definitely a bad file.
	if (InDDSSize < sizeof(DDSHeader))
	{
		//UE_LOGF(LogDDSFile, Display, "Too few bytes to read DDS header");
		*OutError = EDDSError::IoError;
		return nullptr;
	}
	
	// We've now established that InDDSSize >= sizeof(DDSHeader),
	// so we can read that much for sure.
	// It's slightly more convenient to work with an end pointer here.
	const uint8* ReadEnd = InDDS + InDDSSize;
	const uint8* ReadCursor = InDDS;

	memcpy(&DDSHeader, ReadCursor, sizeof(DDSHeader));
	ReadCursor += sizeof(DDSHeader);

	if (DDSHeader.Magic != DDS_MAGIC)
	{
		//UE_LOGF(LogDDSFile, Display, "Not a DDS file.");
		*OutError = EDDSError::NotADds;
		return nullptr;
	}

	// Do we need to read a dx10 header?
	const FDDSPixelFormat& ddpf = DDSHeader.ddspf;
	if ((ddpf.flags & DDPF_FOURCC) && ddpf.fourCC == DX10_MAGIC)
	{
		if (ReadEnd - ReadCursor < sizeof(DDS10Header))
		{
			UE_LOGF(LogDDSFile, Display, "Failed to read DX10 DDS header");
			*OutError = EDDSError::IoError;
			return nullptr;
		}

		memcpy(&DDS10Header, ReadCursor, sizeof(DDS10Header));
		ReadCursor += sizeof(DDS10Header);
	}

	FDDSFile* DDS = new FDDSFile();

	*OutError = ParseHeader(DDS, &DDSHeader, &DDS10Header);
	if (*OutError == EDDSError::OK)
	{
		if (InMipReadMode == UE::DDS::EDDSReadMipMode::Full)
		{
			*OutError = ReadPayload(DDS, ReadCursor, ReadEnd);
		}
		else if(InMipReadMode == UE::DDS::EDDSReadMipMode::HeaderWithMipInfo)
		{
			const FDXGIFormatInfo* FormatInfo = DXGIFormatGetInfo(DDS->DXGIFormat);

			DDS->Mips.Empty();
			DDS->Mips.SetNumZeroed(DDS->ArraySize * DDS->MipCount);

			// Fill mips info without allocating storage
			InitMips(DDS, FormatInfo);
		}
	}

	if (*OutError != EDDSError::OK)
	{
		delete DDS;
		return nullptr;
	}

	return DDS;
 }
 
bool FDDSFile::IsADDS(const uint8* InDDS, int64 InDDSSize)
{
	FDDSHeaderWithMagic DDSHeader = {};
	FDDSHeaderDX10 DDS10Header = {};

	// If we don't even have enough bytes for this to contain a valid header,
	// definitely a bad file.
	if (InDDSSize < sizeof(DDSHeader))
	{
		return false;
	}
	
	// We've now established that InDDSSize >= sizeof(DDSHeader),
	// so we can read that much for sure.
	// It's slightly more convenient to work with an end pointer here.
	const uint8* ReadEnd = InDDS + InDDSSize;
	const uint8* ReadCursor = InDDS;

	memcpy(&DDSHeader, ReadCursor, sizeof(DDSHeader));
	ReadCursor += sizeof(DDSHeader);

	if (DDSHeader.Magic != DDS_MAGIC)
	{
		return false;
	}

	// Do we need to read a dx10 header?
	const FDDSPixelFormat& ddpf = DDSHeader.ddspf;
	if ((ddpf.flags & DDPF_FOURCC) && ddpf.fourCC == DX10_MAGIC)
	{
		if (ReadEnd - ReadCursor < sizeof(DDS10Header))
		{
			return false;
		}

		memcpy(&DDS10Header, ReadCursor, sizeof(DDS10Header));
		ReadCursor += sizeof(DDS10Header);
	}

	return true;
}

//
// Write to an archive (i.e. file)
//
EDDSError FDDSFile::WriteDDS(TArray64<uint8>& OutDDS, EDDSFormatVersion InFormatVersion)
{
	// Validate before we save
	EDDSError ValidateErr = Validate();
	if (ValidateErr != EDDSError::OK)
	{
		return ValidateErr;
	}

	// Preallocate enough memory for all mips and the largest header option
	int64 PreallocSize = sizeof(FDDSHeaderWithMagic) + sizeof(FDDSHeaderDX10);
	for (auto& Mip : Mips)
	{
		PreallocSize += Mip.DataSize;
	}
	OutDDS.Empty();
	OutDDS.Reserve(PreallocSize);

	// We can change format and dimension when writing in D3D9 mode, so keep track of
	// what we're going to write to the file.
	int32 EffectiveDimension = Dimension;
	EDXGIFormat EffectiveFormat = DXGIFormat;

	bool bIsCubemap = (this->CreateFlags & CREATE_FLAG_CUBEMAP) != 0;
	uint32 DepthFlag = (this->Dimension == 3) ? 0x800000 : 0; // DDSD_DEPTH
	uint32 WriteArraySize = bIsCubemap ? this->ArraySize / 6 : this->ArraySize; 

	uint32 Caps2 = 0;
	if (this->Dimension == 3) 
	{
		Caps2 |= DDSCAPS2_VOLUME;
	}
	if (bIsCubemap) 
	{
		Caps2 |= 0xFE00; // DDSCAPS2_CUBEMAP*
	}

	// We always try to find a D3D9 pixel format matching our DXGI Format for export.
	// In D3D10 mode, nothing is done with this information.
	// In Auto mode, we consider D3D9 pixel formats when we consider them widely supported,
	// which are those with AutoD3D9 >= 1.
	// In D3D9 mode, we'll take any D3D9 pixel format that works, even if they get fairly
	// obscure.
	int MinAutoD3d9 = 1; // Only consider D3d9 for pixel formats that have an AutoD3d9 value >= this
	if (InFormatVersion == EDDSFormatVersion::D3D9)
	{
		// In force-D3D9 mode, don't be picky about which formats we consider viable.
		MinAutoD3d9 = 0;

		// D3D9 format DDS can't represent sRGB-ness, so strip sRGB flag from format.
		EffectiveFormat = DXGIFormatRemoveSRGB(EffectiveFormat);
	}

	// Look up how to represent effective format as D3D9 bitmasks format
	// (if multiple choices, pick the first we find)
	const FBitmaskToDXGI* D3d9BitmaskFormat = nullptr;
	for (size_t i = 0; i < sizeof(BitmaskToDXGITable) / sizeof(*BitmaskToDXGITable); ++i)
	{
		if (BitmaskToDXGITable[i].Format == EffectiveFormat && BitmaskToDXGITable[i].AutoD3d9 >= MinAutoD3d9)
		{
			D3d9BitmaskFormat = &BitmaskToDXGITable[i];
			break;
		}
	}

	// Look up how to represent effective format as D3D9 FOURCC format
	// (if multiple choices, pick the first we find)
	uint32 D3d9FourCC = 0;
	for (size_t i = 0; i < sizeof(FOURCCToDXGITable) / sizeof(*FOURCCToDXGITable); i++)
	{
		if (FOURCCToDXGITable[i].Format == EffectiveFormat && FOURCCToDXGITable[i].AutoD3d9 >= MinAutoD3d9)
		{
			D3d9FourCC = FOURCCToDXGITable[i].fourcc;
			break;
		}
	}

	// In D3D9 mode, there's some extra checks because a few things just aren't representable as D3D9.
	if (InFormatVersion == EDDSFormatVersion::D3D9)
	{
		// If we found neither a bitmask format nor FourCC, we don't know how to save this pixel format for D3D9.
		if (!D3d9BitmaskFormat && !D3d9FourCC)
		{
			UE_LOGF(LogDDSFile, Warning, "Unsupported pixel format %ls for D3D9 DDS.", DXGIFormatGetName(EffectiveFormat));
			return EDDSError::BadPixelFormat;
		}

		if (ArraySize != 1)
		{
			UE_LOGF(LogDDSFile, Warning, "D3D9 .DDS does not support arrays.");
			return EDDSError::BadImageDimension;
		}

		if (EffectiveDimension == 1)
		{
			// D3D9 DDS can't do real 1D, it just implicitly turns it into 2D.
			// Height is already 1 (validate checks that), so in D3D9 mode we just
			// bump the effective dimension to 2D and write a Nx1 pixel image instead.
			EffectiveDimension = 2;
		}
	}

	// Set up the DDS header
	FDDSHeaderWithMagic DDSHeader = 
	{
		DDS_MAGIC,
		124, // size value. Required to be 124
		DDSD_CAPS | DDSD_HEIGHT | DDSD_WIDTH | DDSD_PIXELFORMAT | DDSD_MIPMAPCOUNT | DepthFlag,
		this->Height,
		this->Width,
		0, // pitch or linear size
		this->Depth,
		this->MipCount,
		{}, // reserved U32's
		// DDSPF (DDS PixelFormat)
		{
			32, // size, must be 32
			DDPF_FOURCC, // DDPF_FOURCC
			DX10_MAGIC, // Set up for writing as D3D10-format file
			0,0,0,0,0 // Mask and bit counts, if used, are set up below.
		},
		DDSCAPS_COMPLEX | DDSCAPS_TEXTURE | DDSCAPS_MIPMAP,
		Caps2,
		0,
		0,
		0
	};

	uint32 ResourceDimension = RESOURCE_DIMENSION_TEXTURE1D + (EffectiveDimension - 1);
	uint32 MiscFlags = bIsCubemap ? RESOURCE_MISC_TEXTURECUBE : 0;
	FDDSHeaderDX10 DX10Header = 
	{
		(uint32)EffectiveFormat, // DXGI_FORMAT
		ResourceDimension, 
		MiscFlags, 
		WriteArraySize,
		0, // DDS_ALPHA_MODE_UNKNOWN
	};

	// So far, we set up the header for a D3D10-format DDS. Check if we should write
	// as D3D9-format instead.
	//
	// In format=D3D9 mode, we did some extra validation earlier to ensure that we can always
	// write D3D9-format files once we get here, else we would've returned an error earlier.
	// In Auto mode, we check if writing a D3D9-format file is viable, and if so, change our
	// pixel format description to a D3D9 one (which then makes us write a D3D9 DDS).
	if (InFormatVersion != EDDSFormatVersion::D3D10 && EffectiveDimension >= 2 && ArraySize == 1)
	{
		// Sizes are OK.
		// If we have a suitable D3D9 pixel format, writing as D3D9-format DDS is an option.
		FDDSPixelFormat* PixelFmt = &DDSHeader.ddspf;

		if (D3d9BitmaskFormat)
		{
			// Can write as D3D9 bitmask format, so do it!
			PixelFmt->flags = D3d9BitmaskFormat->Flags;
			PixelFmt->fourCC = 0;
			PixelFmt->RGBBitCount = D3d9BitmaskFormat->Bits;
			PixelFmt->RBitMask = D3d9BitmaskFormat->RMask;
			PixelFmt->GBitMask = D3d9BitmaskFormat->GMask;
			PixelFmt->BBitMask = D3d9BitmaskFormat->BMask;
			PixelFmt->ABitMask = D3d9BitmaskFormat->AMask;
		}
		else if (D3d9FourCC != 0)
		{
			PixelFmt->fourCC = D3d9FourCC;
		}

		// If neither of the above two cases were true, continue on with the header we already have,
		// which corresponds to a D3D10 DDS.
		//
		// In Auto mode, this happens when we didn't find a suitable D3D9-esque pixel format to write.
		// In D3D9 mode, this should never happen.
	}

	// Write the headers
	OutDDS.Append((const uint8*)&DDSHeader, sizeof(DDSHeader));
	if (DDSHeader.ddspf.fourCC == DX10_MAGIC)
	{
		check(InFormatVersion != EDDSFormatVersion::D3D9); // If we try to write a D3D10 header despite being in D3D9 mode, something went wrong.
		OutDDS.Append((const uint8*)&DX10Header, sizeof(DX10Header));
	}
	
	// Now go through all subresources in standard order and write them out
	// Need to write them one by one even though we allocate them as a
	// contiguous block if we do it ourselves, because of
	// CREATE_FLAG_NO_MIP_STORAGE_ALLOC.
	for (auto &Mip : this->Mips)
	{
		OutDDS.Append(Mip.Data, Mip.DataSize);
	}

	return EDDSError::OK;
}

void FDDSFile::ConvertRGBXtoRGBA()
{
	// change X8 formats to A8 :
	if ( DXGIFormat == EDXGIFormat::R8G8B8X8 )
	{
		DXGIFormat = EDXGIFormat::R8G8B8A8_UNORM_SRGB;
	}
	else if ( DXGIFormat == EDXGIFormat::B8G8R8X8_UNORM )
	{
		DXGIFormat = EDXGIFormat::B8G8R8A8_UNORM;
	}
	else if ( DXGIFormat == EDXGIFormat::B8G8R8X8_UNORM_SRGB )
	{
		DXGIFormat = EDXGIFormat::B8G8R8A8_UNORM_SRGB;
	}
	else if ( DXGIFormat == EDXGIFormat::B8G8R8X8_TYPELESS )
	{
		DXGIFormat = EDXGIFormat::B8G8R8A8_TYPELESS;
	}
	else
	{
		// not an X8 format
		return;
	}

	// stuff 255 in A :
	for (auto& Mip : Mips)
	{
		// Loop over pixels. Data in DDS mips is densely packed!
		uint8* PixelData = Mip.Data;
		const int64 DataSize = Mip.DataSize;
		for (int64 i = 0; i < DataSize; i += 4)
		{
			PixelData[3] = 0xFF;
		}
	}
}

void FDDSFile::ConvertChannelOrder(EChannelOrder InTargetOrder)
{
	// Is this one of the few RGBA/BGRA formats we can convert?
	int FormatIndex = DXGIFormatFindInTable(DXGIFormatRGBATable, DXGIFormat);
	if (FormatIndex < 0)
	{
		// Nope, not one of those formats, leave it alone.
		return;
	}

	// Figure out which channel order we currently are based on whether the index
	// is even or odd.
	EChannelOrder CurrentOrder = (FormatIndex & 1) ? EChannelOrder::BGRA : EChannelOrder::RGBA;
	if (CurrentOrder == InTargetOrder)
	{
		// Channel order is already target order, nothing to do!
		return;
	}

	// Change format to the opposite in the pair, then fix the pixel data
	// by swapping R and B.
	DXGIFormat = DXGIFormatRGBATable[FormatIndex ^ 1];

	for (auto& Mip : Mips)
	{
		// Loop over pixels. Data in DDS mips is densely packed!
		uint8* PixelData = Mip.Data;
		const int64 DataSize = Mip.DataSize;
		for (int64 i = 0; i < DataSize; i += 4)
		{
			// RGBA <-> BGRA swaps B and R and leaves G and A alone.
			// This can be done more efficiently but keep it simple here.
			uint8 t = PixelData[i];
			PixelData[i] = PixelData[i + 2];
			PixelData[i + 2] = t;
		}
	}
}

void FDDSFile::FillMip(const FImageView & FromImage, int MipIndex)
{
	check( MipIndex < Mips.Num() );
	const FDDSMip & ToMip = Mips[MipIndex];

	check( FromImage.GetNumPixels() == ToMip.Width * ToMip.Height * (int64) ToMip.Depth );
	check( FromImage.GetImageSizeBytes() == ToMip.DataSize );
		
	memcpy(ToMip.Data,FromImage.RawData,ToMip.DataSize);
}

// Blit pixels from DDS mip to Image
// return false if no pixel format conversion is supported
bool FDDSFile::GetMipImage(const FImageView & ToImage, int MipIndex) const
{
	check( MipIndex < Mips.Num() );
	const FDDSMip & FromMip = Mips[MipIndex];

	check( ToImage.GetNumPixels() == FromMip.Width * FromMip.Height * (int64) FromMip.Depth );

	// before BlitMip you should have done ConvertChannelOrder to BGRA and ConvertRGBXtoRGBA
	// so we should only see BGRA (no RGBA or BGRX) here

	if ( ToImage.Format == ERawImageFormat::BGRA8 && DXGIFormat == EDXGIFormat::B8G8R8 )
	{
		const uint8 * FmPtr = FromMip.Data;
		uint8 * ToPtr = (uint8 *)ToImage.RawData;
		int64 Count = ToImage.GetNumPixels();
		check( ToImage.GetImageSizeBytes() == Count*4 );
		check( FromMip.DataSize == Count*3 );
		while(Count--)
		{
			*ToPtr++ = *FmPtr++;
			*ToPtr++ = *FmPtr++;
			*ToPtr++ = *FmPtr++;
			*ToPtr++ = 0xFF;
		}
		return true;
	}
	else if ( ToImage.Format == ERawImageFormat::BGRA8 && DXGIFormat == EDXGIFormat::R8G8B8 )
	{
		const uint8 * FmPtr = FromMip.Data;
		uint8 * ToPtr = (uint8 *)ToImage.RawData;
		int64 Count = ToImage.GetNumPixels();
		check( ToImage.GetImageSizeBytes() == Count*4 );
		check( FromMip.DataSize == Count*3 );
		while(Count--)
		{
			*ToPtr++ = FmPtr[2];
			*ToPtr++ = FmPtr[1];
			*ToPtr++ = FmPtr[0];
			*ToPtr++ = 0xFF;
			FmPtr += 3;
		}
		return true;
	}
	else if ( ToImage.Format == ERawImageFormat::BGRA8 && (
		DXGIFormat == EDXGIFormat::R8G8_TYPELESS ||
		DXGIFormat == EDXGIFormat::R8G8_UNORM ||
		DXGIFormat == EDXGIFormat::R8G8_UINT ||
		DXGIFormat == EDXGIFormat::R8G8_SNORM ||
		DXGIFormat == EDXGIFormat::R8G8_SINT ) )
	{
		const uint8 * FmPtr = FromMip.Data;
		uint8 * ToPtr = (uint8 *)ToImage.RawData;
		int64 Count = ToImage.GetNumPixels();
		check( ToImage.GetImageSizeBytes() == Count*4 );
		check( FromMip.DataSize == Count*2 );
		while(Count--)
		{
			*ToPtr++ = 0;
			*ToPtr++ = FmPtr[1];
			*ToPtr++ = FmPtr[0];
			*ToPtr++ = 0xFF;
			FmPtr += 2;
		}
		check( ToPtr == (uint8 *)ToImage.RawData + ToImage.GetImageSizeBytes() );
		check( FmPtr == FromMip.Data + FromMip.DataSize );
		return true;
	}
	else if ( ToImage.GetImageSizeBytes() == FromMip.DataSize )
	{	
		bool bIsExactMatch;
		ERawImageFormat::Type DXGIToRawFormat = DXGIFormatGetClosestRawFormat(DXGIFormat,&bIsExactMatch);

		if ( DXGIToRawFormat == ToImage.Format )
		{
			if ( ! bIsExactMatch )
			{
				UE_LOGF(LogDDSFile, Warning, "DDS BlitMip DXGIFormat isn't exact match fmt %d=(%ls) to (%ls)", \
							 int(DXGIFormat), DXGIFormatGetName(DXGIFormat),ERawImageFormat::GetName(ToImage.Format));
				// but do it anyway
			}
		
			memcpy(ToImage.RawData,FromMip.Data,FromMip.DataSize);
			return true;
		}
		else 
		{
			// formats mismatch but are same size
			// very speculative, try it anyway, but log :
		
			UE_LOGF(LogDDSFile, Warning, "DDS BlitMip DXGIFormat incompatible but same size! may be junk!  fmt %d=(%ls) to (%ls)", \
						 int(DXGIFormat), DXGIFormatGetName(DXGIFormat),ERawImageFormat::GetName(ToImage.Format));

			memcpy(ToImage.RawData,FromMip.Data,FromMip.DataSize);
			return true;
		}
	}
	else
	{
		// no supported conversion
		
		UE_LOGF(LogDDSFile, Warning, "DDS BlitMip DXGIFormat cannot blit!  %d (%ls) to (%ls)", \
					 int(DXGIFormat), DXGIFormatGetName(DXGIFormat),ERawImageFormat::GetName(ToImage.Format));

		return false;
	}
}

struct FDXGIFormatRawFormatMapping
{
	EDXGIFormat DXGIFormat;
	ERawImageFormat::Type RawFormat;
	bool bIsExactMatch;
};

FDXGIFormatRawFormatMapping DXGIRawFormatMap[] =
{
	{ EDXGIFormat::R32G32B32A32_FLOAT, ERawImageFormat::RGBA32F, true },
	{ EDXGIFormat::R16G16B16A16_FLOAT, ERawImageFormat::RGBA16F, true },
	{ EDXGIFormat::R32_FLOAT,  ERawImageFormat::R32F, true },
	{ EDXGIFormat::R16_FLOAT,  ERawImageFormat::R16F, true },

	// R32G32B32_FLOAT not supported
	// R32G32_FLOAT not supported
	// R16G16_FLOAT not supported

	{ EDXGIFormat::R16G16B16A16_UNORM, ERawImageFormat::RGBA16, true },
	{ EDXGIFormat::R16G16B16A16_UINT,  ERawImageFormat::RGBA16, true },
	{ EDXGIFormat::R16G16B16A16_SNORM, ERawImageFormat::RGBA16, false },
	{ EDXGIFormat::R16G16B16A16_SINT,  ERawImageFormat::RGBA16, false },

	{ EDXGIFormat::R8G8B8A8_TYPELESS,  ERawImageFormat::BGRA8, false },
	{ EDXGIFormat::R8G8B8A8_UNORM,  ERawImageFormat::BGRA8, false },
	{ EDXGIFormat::R8G8B8A8_UNORM_SRGB,  ERawImageFormat::BGRA8, false },
	{ EDXGIFormat::R8G8B8A8_UINT,  ERawImageFormat::BGRA8, false },
	{ EDXGIFormat::R8G8B8A8_SNORM,  ERawImageFormat::BGRA8, false },
	{ EDXGIFormat::R8G8B8A8_SINT,  ERawImageFormat::BGRA8, false },
	
	{ EDXGIFormat::B8G8R8A8_UNORM,  ERawImageFormat::BGRA8, true },
	{ EDXGIFormat::B8G8R8X8_UNORM,  ERawImageFormat::BGRA8, false },
	
	{ EDXGIFormat::B8G8R8A8_TYPELESS,  ERawImageFormat::BGRA8, true },
	{ EDXGIFormat::B8G8R8A8_UNORM_SRGB,  ERawImageFormat::BGRA8, true },
	{ EDXGIFormat::B8G8R8X8_TYPELESS,  ERawImageFormat::BGRA8, false },
	{ EDXGIFormat::B8G8R8X8_UNORM_SRGB,  ERawImageFormat::BGRA8, false },

	{ EDXGIFormat::R32_UINT,  ERawImageFormat::G16, false },
	{ EDXGIFormat::R32_SINT,  ERawImageFormat::G16, false },
	
	{ EDXGIFormat::R16_TYPELESS,  ERawImageFormat::G16, true },
	{ EDXGIFormat::R16_UNORM,  ERawImageFormat::G16, true },
	{ EDXGIFormat::R16_UINT,  ERawImageFormat::G16, true },
	{ EDXGIFormat::R16_SNORM,  ERawImageFormat::G16, false },
	{ EDXGIFormat::R16_SINT,  ERawImageFormat::G16, false },
	
	{ EDXGIFormat::R8_TYPELESS,  ERawImageFormat::G8, true },
	{ EDXGIFormat::R8_UNORM,  ERawImageFormat::G8, true },
	{ EDXGIFormat::R8_UINT,  ERawImageFormat::G8, true },
	{ EDXGIFormat::R8_SNORM,  ERawImageFormat::G8, false },
	{ EDXGIFormat::R8_SINT,  ERawImageFormat::G8, false },
	{ EDXGIFormat::A8_UNORM,  ERawImageFormat::G8, false },

	{ EDXGIFormat::B8G8R8, ERawImageFormat::BGRA8, false },
	{ EDXGIFormat::R8G8B8, ERawImageFormat::BGRA8, false },
	{ EDXGIFormat::R8G8B8X8, ERawImageFormat::BGRA8, false },
	
	{ EDXGIFormat::R8G8_TYPELESS, ERawImageFormat::BGRA8, false },
	{ EDXGIFormat::R8G8_UNORM, ERawImageFormat::BGRA8, false },
	{ EDXGIFormat::R8G8_UINT, ERawImageFormat::BGRA8, false },
	{ EDXGIFormat::R8G8_SNORM, ERawImageFormat::BGRA8, false },
	{ EDXGIFormat::R8G8_SINT, ERawImageFormat::BGRA8, false },

	// terminates list :
	{ EDXGIFormat::UNKNOWN, ERawImageFormat::Invalid, false}
};

ERawImageFormat::Type DXGIFormatGetClosestRawFormat(EDXGIFormat fmt, bool * pIsExactMatch)
{
	for(const FDXGIFormatRawFormatMapping * Mapping = DXGIRawFormatMap;Mapping->DXGIFormat != EDXGIFormat::UNKNOWN;++Mapping)
	{
		if ( Mapping->DXGIFormat == fmt )
		{
			if ( pIsExactMatch )
			{
				*pIsExactMatch = Mapping->bIsExactMatch;
			}
			return Mapping->RawFormat;
		}
	}

	return ERawImageFormat::Invalid;
}

EDXGIFormat DXGIFormatFromRawFormat(ERawImageFormat::Type RawFormat,EGammaSpace GammaSpace)
{
	switch(RawFormat)
	{
	case ERawImageFormat::G8:		return EDXGIFormat::R8_UNORM; // would like to encode Gamma, but no way to do so
	case ERawImageFormat::BGRA8:
		{
		if ( GammaSpace == EGammaSpace::Linear)
			return EDXGIFormat::B8G8R8A8_UNORM;
		else
			return EDXGIFormat::B8G8R8A8_UNORM_SRGB;
		}
	case ERawImageFormat::BGRE8:	return EDXGIFormat::B8G8R8A8_UNORM; // no way to indicate this is BGRE
	case ERawImageFormat::RGBA16:	return EDXGIFormat::R16G16B16A16_UNORM;
	case ERawImageFormat::RGBA16F:	return EDXGIFormat::R16G16B16A16_FLOAT;
	case ERawImageFormat::RGBA32F:	return EDXGIFormat::R32G32B32A32_FLOAT;
	case ERawImageFormat::G16:		return EDXGIFormat::R16_UNORM;
	case ERawImageFormat::R16F:		return EDXGIFormat::R16_FLOAT;
	case ERawImageFormat::R32F:		return EDXGIFormat::R32_FLOAT;
	default:
		return EDXGIFormat::UNKNOWN;
	}
}

} } // end UE::DDS namespace
