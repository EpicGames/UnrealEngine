// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageCore.h"
#include "Modules/ModuleManager.h"
#include "Async/ParallelFor.h"
#include "Serialization/CompactBinaryWriter.h"
#include "ColorManagement/TransferFunctions.h"
#include "ColorManagement/ColorSpace.h"
#include "ImageParallelFor.h"
#include "Tasks/Task.h"

DEFINE_LOG_CATEGORY_STATIC(LogImageCore, Log, All);

IMPLEMENT_MODULE(FDefaultModuleImpl, ImageCore);


/* Local helper functions
 *****************************************************************************/
 
static uint8 Requantize16to8(const uint16 In)
{
	// same as QuantizeRound(In/65535.f);
    uint32 Ret = ( (uint32)In * 255 + 32768 + 127 )>>16;
	checkSlow( Ret <= 255 );
	return (uint8)Ret;
}

static const TCHAR * GammaSpaceGetName(EGammaSpace GammaSpace)
{
	switch(GammaSpace)
	{
	case EGammaSpace::Linear: return TEXT("Linear");
	case EGammaSpace::Pow22: return TEXT("Pow22");
	case EGammaSpace::sRGB: return TEXT("sRGB");
	default: return TEXT("Invalid");
	}
}


/**
 * Initializes storage for an image.
 *
 * @param Image - The image to initialize storage for.
 */
static void InitImageStorage(FImage& Image)
{
	//TRACE_CPUPROFILER_EVENT_SCOPE(Texture.InitImageStorage);

	check( Image.IsImageInfoValid() );

	// if the Image.RawData is already the right size, this does not re-allocate
	int64 NumBytes = Image.GetImageSizeBytes();
	Image.RawData.Empty(NumBytes);
	Image.RawData.AddUninitialized(NumBytes);
}

/**
 * Compute number of jobs to use for ParallelFor
 *
 * @param OutNumItemsPerJob - filled with num items per job; NumJobs*OutNumItemsPerJob >= NumItems
 * @param NumItems - total num items
 * @param MinNumItemsPerJob - jobs will do at least this many items each
 * @param MinNumItemsForAnyJobs - (optional) if NumItems is less than this, no parallelism are used
 * @return NumJobs
 */
static inline int32 ParallelForComputeNumJobs(int64 & OutNumItemsPerJob,int64 NumItems,int64 MinNumItemsPerJob,int64 MinNumItemsForAnyJobs = 0)
{
	if ( NumItems <= FMath::Max(MinNumItemsPerJob,MinNumItemsForAnyJobs) )
	{
		OutNumItemsPerJob = NumItems;
		return 1;
	}
	
	// Always use ParallelFor with "Unbalanced"  so it does not do weird things to your job count

	const int32 NumWorkers = FMath::Max(1, FTaskGraphInterface::Get().GetNumWorkerThreads());
	int32 NumJobs = IntCastChecked<int32>(NumItems / MinNumItemsPerJob); // round down

	// NumJobs is now the maximum number of jobs we would want to do
	//	(eg. one for each small packet of MinNumItemsPerJob)
	//	but that's often way too many

	// Assign 1,2, or 3 jobs per worker
	// never make jobs that will be smaller than MinNumItemsPerJob (eg. NumJobs is only adjusted downward)
	// more jobs per worker provides load balancing, if one worker stalls others can pick up the jobs he didn't do

	if ( NumJobs >= NumWorkers*3 )
	{
		// enough jobs to evenly distribute 3 per worker
		// reduce job count to provide some slack for stalling workers
		NumJobs = NumWorkers*3 - 1;
	}
	else if ( NumJobs >= NumWorkers*2 )
	{
		// enough jobs to evenly distribute 2 per worker
		NumJobs = NumWorkers*2 - 1;
	}
	else
	{
		// not enough jobs to evenly distribute more than 1 per worker
		NumJobs = FMath::Clamp(NumJobs, int32(1), NumWorkers); 
	}

	if ( NumJobs > 16 )
	{
		// add some slack; assume one thread will not be available
		//	slightly under-subscribing reduces the chance of a much higher latency
		NumJobs--;
	}

	OutNumItemsPerJob = (NumItems + NumJobs-1) / NumJobs; // round up
	check( NumJobs*OutNumItemsPerJob >= NumItems );

	return NumJobs;
}

static constexpr int64 MinPixelsPerJob = 16384;
// Surfaces of VT tile size or smaller will not parallelize at all :
static constexpr int64 MinPixelsForAnyJob = 136*136;
// Jobs split on pixel count (not rows) are aligned to PixelsPerJobAlignment :
static constexpr int64 PixelsPerJobAlignment = 64; // must be power of 2

IMAGECORE_API int32 ImageParallelForComputeNumJobsForPixels(int64 & OutNumPixelsPerJob,int64 NumPixels)
{
	check( NumPixels > 0 );

	int32 NumJobs = ParallelForComputeNumJobs(OutNumPixelsPerJob,NumPixels,MinPixelsPerJob,MinPixelsForAnyJob);
	
	if ( NumJobs > 1 )
	{
		// align up to PixelsPerJobAlignment :
		//	(PixelsPerJobAlignment should be power of 2)
		OutNumPixelsPerJob = (OutNumPixelsPerJob + PixelsPerJobAlignment-1) & (~ (PixelsPerJobAlignment-1) );
		// recompute NumJobs :
		NumJobs = (NumPixels + OutNumPixelsPerJob-1) / OutNumPixelsPerJob;
	}

	check( NumJobs*OutNumPixelsPerJob >= NumPixels );
	check( (NumJobs-1)*OutNumPixelsPerJob < NumPixels );

	return NumJobs;
}

IMAGECORE_API int32 ImageParallelForComputeNumJobsForRows(int32 & OutNumItemsPerJob,int64 SizeX,int64 SizeY)
{
	check( SizeX > 0 && SizeY > 0 );

	int64 NumPixels = SizeX*SizeY;
	int64 OutNumPixelsPerJob;
	int64 NumJobs1 = ParallelForComputeNumJobs(OutNumPixelsPerJob,NumPixels,MinPixelsPerJob,MinPixelsForAnyJob);
	int64 OutNumItemsPerJob64 = (SizeY + NumJobs1-1) / NumJobs1;
	
	// recompute NumJobs :
	int64 NumJobs = (SizeY + OutNumItemsPerJob64-1) / OutNumItemsPerJob64;

	check( OutNumItemsPerJob64*NumJobs >= SizeY );
	check( OutNumItemsPerJob64*(NumJobs-1) < SizeY );

	OutNumItemsPerJob = IntCastChecked<int32>(OutNumItemsPerJob64);
	return IntCastChecked<int32>(NumJobs);
}

template <typename Lambda>
static void ParallelLoop(const TCHAR* DebugName, int32 NumJobs, int64 TexelsPerJob, int64 NumTexels, const Lambda& Func)
{
	if ( NumJobs <= 1 )
	{
		// special case for non-parallel
		//	this is mainly to avoid making insights traces for this case
		//	would be nice if ParallelFor just had a very early out for this

		for (int64 TexelIndex = 0; TexelIndex < NumTexels; ++TexelIndex)
		{
			Func(TexelIndex);
		}
	}
	else
	{
		ParallelFor(DebugName, NumJobs, 1, [=](int64 JobIndex)
		{
			const int64 StartIndex = JobIndex * TexelsPerJob;
			const int64 EndIndex = FMath::Min(StartIndex + TexelsPerJob, NumTexels);
			for (int64 TexelIndex = StartIndex; TexelIndex < EndIndex; ++TexelIndex)
			{
				Func(TexelIndex);
			}
		}, EParallelForFlags::Unbalanced);
	}
}

// ParallelOr : call Func() on all texels ; returns true if Func is true for any texel
template <typename Lambda>
static bool ParallelOr(const TCHAR* DebugName, int32 NumJobs, int64 TexelsPerJob, int64 NumTexels, const Lambda& Func)
{
	// merged_bool starts false, is set to true if any thread sees a true
	std::atomic<bool> merged_bool(false);

	ParallelFor(DebugName, NumJobs, 1, [&](int64 JobIndex)
	{
		const int64 StartIndex = JobIndex * TexelsPerJob;
		const int64 EndIndex = FMath::Min(StartIndex + TexelsPerJob, NumTexels);
		for (int64 TexelIndex = StartIndex; TexelIndex < EndIndex; ++TexelIndex)
		{
			if ( Func(TexelIndex) )
			{
				merged_bool.store(true,std::memory_order_release);
				return;
			}
		}
		// if Func was really slow, you might want to check if merged_bool was set to true on some other thread
		//	and terminate our loop when that is true
		//  but that causes cache traffic for the shared variable, so should not be done in tight loops
	}, EParallelForFlags::Unbalanced);

	bool ret = merged_bool.load(std::memory_order_acquire);
	return ret;
}

template <int64 TexelsPerVec, typename LambdaVec, typename Lambda>
static void ParallelLoop(const TCHAR* DebugName, int32 NumJobs, int64 TexelsPerJob, int64 NumTexels, const LambdaVec& FuncVec, const Lambda& Func)
{
	static_assert(FMath::IsPowerOfTwo(TexelsPerVec), "TexelsPerVec must be power of 2");

	ParallelFor(DebugName, NumJobs, 1, [=](int64 JobIndex)
	{
		const int64 StartIndex = JobIndex * TexelsPerJob;
		const int64 EndIndex = FMath::Min(StartIndex + TexelsPerJob, NumTexels);
		int64 TexelIndex = StartIndex;
#if PLATFORM_CPU_X86_FAMILY
		const int64 TexelCount = EndIndex - StartIndex;
		const int64 EndIndexVec = StartIndex + (TexelCount & ~(TexelsPerVec - 1));
		while (TexelIndex < EndIndexVec)
		{
			FuncVec(TexelIndex);
			TexelIndex += TexelsPerVec;
		}
#endif
		while (TexelIndex < EndIndex)
		{
			Func(TexelIndex);
			++TexelIndex;
		}
	}, EParallelForFlags::Unbalanced);
}

// 2xU16 is not a proper supported Image format
//	DestImage should be BGRA8 so it's the correct 4 bytes per pel
//	we write 2xU16 to it
IMAGECORE_API void FImageCore::CopyImageTo2U16(const FImageView & SrcImage,const FImageView & DestImage)
{
	check( SrcImage.GetNumPixels() == DestImage.GetNumPixels() );
	check( DestImage.Format == ERawImageFormat::BGRA8 );
	check(SrcImage.IsImageInfoValid());
	check(DestImage.IsImageInfoValid());

	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.CopyImageTo2U16);
			
	UE_LOGF(LogImageCore,Verbose,"CopyImageTo2U16: %ls %ls -> %ls %ls %dx%d",
		ERawImageFormat::GetName(SrcImage.Format),
		GammaSpaceGetName(SrcImage.GammaSpace),
		ERawImageFormat::GetName(DestImage.Format),
		GammaSpaceGetName(DestImage.GammaSpace),
		SrcImage.SizeX,SrcImage.SizeY);

	const int64 NumTexels = SrcImage.GetNumPixels();
	int64 TexelsPerJob;
	int32 NumJobs = ImageParallelForComputeNumJobsForPixels(TexelsPerJob,NumTexels);

	if (SrcImage.Format == ERawImageFormat::RGBA32F)
	{
		// Convert from 32-bit linear floating point.
		const FLinearColor* SrcColors = (const FLinearColor*) SrcImage.RawData;
		uint16* DestColors = (uint16 *) DestImage.RawData;

		ParallelLoop(TEXT("Texture.CopyImageTo2U16.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcColors](int64 TexelIndex)
		{
			FLinearColor Src = SrcColors[TexelIndex];
			uint16* Dst = DestColors + TexelIndex * 2;
			Dst[0] = FColor::QuantizeUNormFloatTo16(Src.R);
			Dst[1] = FColor::QuantizeUNormFloatTo16(Src.G);
		});
	}
	else if (SrcImage.Format == ERawImageFormat::BGRA8 && SrcImage.GammaSpace == EGammaSpace::Linear )
	{
		const FColor* SrcColors = (const FColor*) SrcImage.RawData;
		uint16* DestColors = (uint16*) DestImage.RawData;
		
		ParallelLoop(TEXT("Texture.CopyImageTo2U16.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcColors](int64 TexelIndex)
		{
			FColor Src = SrcColors[TexelIndex];
			uint16* Dst = DestColors + TexelIndex * 2;
			Dst[0] = (Src.R << 8) | Src.R;
			Dst[1] = (Src.G << 8) | Src.G;
		});
	}
	else
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Texture.CopyImageTo2U16.TempLinear);

		// Arbitrary conversion, use 32-bit linear float as an intermediate format.
		// this is unnecessarily expensive to do something like G8 to R16F, but rare
		// if this shows up as a hot spot, identify the formats using this path and add direct conversions between them
		FImage TempImage(SrcImage.SizeX, SrcImage.SizeY, SrcImage.NumSlices, ERawImageFormat::RGBA32F, EGammaSpace::Linear);
		FImageCore::CopyImage(SrcImage, TempImage);
		FImageCore::CopyImageTo2U16(TempImage, DestImage);
	}
}

// Copy Image, swapping RB, SrcImage must be BGRA8 or RGBA16
// Src == Dest is okay
IMAGECORE_API void FImageCore::CopyImageRGBABGRA(const FImageView & SrcImage,const FImageView & DestImage)
{
	check( SrcImage.GetNumPixels() == DestImage.GetNumPixels() );
	check( SrcImage.Format == DestImage.Format );
	const int64 NumTexels = SrcImage.GetNumPixels();

	if ( SrcImage.Format == ERawImageFormat::BGRA8 )
	{	
		const uint8 * SrcColors = (const uint8 *)SrcImage.RawData;
		const uint8 * SrcColorsEnd = SrcColors + SrcImage.GetImageSizeBytes();
		uint8 * DestColors = (uint8 *)DestImage.RawData;
	
		for(;SrcColors<SrcColorsEnd;SrcColors+=4,DestColors+=4)
		{
			// make it work with Src == Dest
			uint8 R = SrcColors[0];
			uint8 B = SrcColors[2];
			DestColors[0] = B;
			DestColors[1] = SrcColors[1];
			DestColors[2] = R;
			DestColors[3] = SrcColors[3];
		}
	}
	else if ( SrcImage.Format == ERawImageFormat::RGBA16 )
	{
		const uint16 * SrcColors = (const uint16 *)SrcImage.RawData;
		const uint16 * SrcColorsEnd = SrcColors + SrcImage.GetImageSizeBytes()/sizeof(uint16);
		uint16 * DestColors = (uint16 * )DestImage.RawData;
	
		for(;SrcColors<SrcColorsEnd;SrcColors+=4,DestColors+=4)
		{
			// make it work with Src == Dest
			uint16 R = SrcColors[0];
			uint16 B = SrcColors[2];
			DestColors[0] = B;
			DestColors[1] = SrcColors[1];
			DestColors[2] = R;
			DestColors[3] = SrcColors[3];
		}
	}
	else
	{
		check(0);
	}
}

IMAGECORE_API void FImageCore::TransposeImageRGBABGRA(const FImageView & Image)
{
	// CopyImageRGBABGRA is okay in place
	CopyImageRGBABGRA(Image,Image);
}

/**
 * Copies an image accounting for format differences. Sizes must match.
 *
 * @param SrcImage - The source image to copy from.
 * @param DestImage - The destination image to copy to.
 */
IMAGECORE_API void FImageCore::CopyImage(const FImageView & SrcImage,const FImageView & DestImage)
{
	// self-calls before the TRACE_CPUPROFILER_EVENT_SCOPE
	// if the calling code is correct, these should not be used
	//	they are a temporary patch in case bad calling code lingers
	if ( SrcImage.IsGammaCorrected() && ! GetFormatNeedsGammaSpace(SrcImage.Format) )
	{
		UE_LOGF(LogImageCore,Warning,"CopyImage: SrcImage has invalid gamma settings %ls %ls",
			ERawImageFormat::GetName(SrcImage.Format),
			GammaSpaceGetName(SrcImage.GammaSpace));

		// if we're given {F32,sRGB} assume they meant {F32,Linear} and go ahead with the copy
		FImageView SrcLinear = SrcImage;
		SrcLinear.GammaSpace = EGammaSpace::Linear;
		FImageCore::CopyImage(SrcLinear,DestImage);
		return;
	}
	else if ( DestImage.IsGammaCorrected() && ! GetFormatNeedsGammaSpace(DestImage.Format) )
	{
		UE_LOGF(LogImageCore,Warning,"CopyImage: DestImage has invalid gamma settings %ls %ls",
			ERawImageFormat::GetName(DestImage.Format),
			GammaSpaceGetName(DestImage.GammaSpace));
			
		// if we're given {F32,sRGB} assume they meant {F32,Linear} and go ahead with the copy
		FImageView DestLinear = DestImage;
		DestLinear.GammaSpace = EGammaSpace::Linear;
		FImageCore::CopyImage(SrcImage,DestLinear);
		return;
	}

	bool bDoTrace = SrcImage.GetNumPixels() > 16384;
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_CONDITIONAL("Texture.CopyImage",bDoTrace);

	check(SrcImage.IsImageInfoValid());
	check(DestImage.IsImageInfoValid());
	check(SrcImage.GetNumPixels() == DestImage.GetNumPixels());
	
	// short cut fast path identical pixels before we do anything else
	if ( SrcImage.Format == DestImage.Format &&
		SrcImage.GammaSpace == DestImage.GammaSpace )
	{
		//TRACE_CPUPROFILER_EVENT_SCOPE(Texture.CopyImage.memcpy);

		int64 Bytes = SrcImage.GetImageSizeBytes();
		check( DestImage.GetImageSizeBytes() == Bytes );
		memcpy(DestImage.RawData,SrcImage.RawData,Bytes);
		return;
	}
	
	UE_LOGF(LogImageCore,Verbose,"CopyImage: %ls %ls -> %ls %ls %dx%d",
		ERawImageFormat::GetName(SrcImage.Format),
		GammaSpaceGetName(SrcImage.GammaSpace),
		ERawImageFormat::GetName(DestImage.Format),
		GammaSpaceGetName(DestImage.GammaSpace),
		SrcImage.SizeX,SrcImage.SizeY);

	// bDestIsGammaCorrected for Dest (Pow22 and sRGB) will encode to sRGB
	// note that gamma correction is only performed for U8 formats
	const bool bDestIsGammaCorrected = DestImage.IsGammaCorrected();
	const int64 NumTexels = SrcImage.GetNumPixels();
	int64 TexelsPerJob;
	int32 NumJobs = ImageParallelForComputeNumJobsForPixels(TexelsPerJob,NumTexels);

	if (SrcImage.Format == ERawImageFormat::RGBA32F)
	{
		// Convert from 32-bit linear floating point.
		const FLinearColor* SrcColors = (const FLinearColor*) SrcImage.RawData;
			
		// if gamma correction is done, it's always *TO* sRGB , not to Pow22
		// bDestIsGammaCorrected always outputs sRGB
		if ( DestImage.GammaSpace == EGammaSpace::Pow22 )
		{
			UE_LOGF(LogImageCore, Warning, "Pow22 should not be used as a Dest GammaSpace.  Pow22 Source should encode to sRGB Dest.");
		}
		
		switch (DestImage.Format)
		{
		case ERawImageFormat::G8:
			{
				// NOTE : blit from RGBA to G8 does NOT grab the gray, it just take the R
				//	this makes blits from R16F/R32F go to G8/G16 and just take the single channel
				uint8* DestLum = (uint8 *) DestImage.RawData;
				if (bDestIsGammaCorrected)
				{
					ParallelLoop(TEXT("Texture.CopyImage.PF"), NumJobs, TexelsPerJob, NumTexels, [DestLum, SrcColors](int64 TexelIndex)
					{
						DestLum[TexelIndex] = SrcColors[TexelIndex].ToFColorSRGB().R;
					});
				}
				else
				{
					ParallelLoop(TEXT("Texture.CopyImage.PF"), NumJobs, TexelsPerJob, NumTexels, [DestLum, SrcColors](int64 TexelIndex)
					{
						DestLum[TexelIndex] = SrcColors[TexelIndex].QuantizeRound().R;
					});
				}
			}
			break;

		case ERawImageFormat::G16:
			{
				// NOTE : blit from RGBA to G16 does NOT grab the gray, it just take the R
				//	this makes blits from R16F/R32F go to G8/G16 and just take the single channel
				uint16* DestLum = (uint16 *) DestImage.RawData;
				ParallelLoop(TEXT("Texture.CopyImage.PF"), NumJobs, TexelsPerJob, NumTexels, [DestLum, SrcColors](int64 TexelIndex)
				{
					DestLum[TexelIndex] = FColor::QuantizeUNormFloatTo16(SrcColors[TexelIndex].R);
				});
			}
			break;

		case ERawImageFormat::BGRA8:
			{
				FColor* DestColors = (FColor *) DestImage.RawData;
				if (bDestIsGammaCorrected)
				{
					ParallelFor(TEXT("Texture.CopyImage.PF"), NumJobs, 1,
						[DestColors, SrcColors, TexelsPerJob, NumTexels](int64 JobIndex)
						{
							const int64 StartIndex = JobIndex * TexelsPerJob;
							const int64 Count = FMath::Min(TexelsPerJob, NumTexels - StartIndex);
							ConvertFLinearColorsToFColorSRGB(&SrcColors[StartIndex], &DestColors[StartIndex], Count);
						}
					, EParallelForFlags::Unbalanced);
				}
				else
				{
					ParallelLoop<4>(TEXT("Texture.CopyImage.PF"), NumJobs, TexelsPerJob, NumTexels,
						[DestColors, SrcColors](int64 TexelIndex)
						{
#if PLATFORM_CPU_X86_FAMILY
							// load 4x RGBA32F
							__m128 Pixel0 = _mm_loadu_ps(&SrcColors[TexelIndex + 0].Component(0));
							__m128 Pixel1 = _mm_loadu_ps(&SrcColors[TexelIndex + 1].Component(0));
							__m128 Pixel2 = _mm_loadu_ps(&SrcColors[TexelIndex + 2].Component(0));
							__m128 Pixel3 = _mm_loadu_ps(&SrcColors[TexelIndex + 3].Component(0));

							// RGBA -> BGRA
							Pixel0 = _mm_shuffle_ps(Pixel0, Pixel0, _MM_SHUFFLE(3, 0, 1, 2));
							Pixel1 = _mm_shuffle_ps(Pixel1, Pixel1, _MM_SHUFFLE(3, 0, 1, 2));
							Pixel2 = _mm_shuffle_ps(Pixel2, Pixel2, _MM_SHUFFLE(3, 0, 1, 2));
							Pixel3 = _mm_shuffle_ps(Pixel3, Pixel3, _MM_SHUFFLE(3, 0, 1, 2));

							// Scale to [0,255]
							__m128 Mul = _mm_set_ps1(255.f);
							__m128 Add = _mm_set_ps1(0.5f);
							Pixel0 = _mm_add_ps(_mm_mul_ps(Pixel0, Mul), Add);
							Pixel1 = _mm_add_ps(_mm_mul_ps(Pixel1, Mul), Add);
							Pixel2 = _mm_add_ps(_mm_mul_ps(Pixel2, Mul), Add);
							Pixel3 = _mm_add_ps(_mm_mul_ps(Pixel3, Mul), Add);

							// Clamp large values at 255. Must be first arg: SSE min_ps(255, NaN) = NaN,
							// but min_ps(NaN, 255) = 255, and we want the NaNs to turn into 0 not 255,
							// for consistency with QuantizeRound.
							Pixel0 = _mm_min_ps(Mul, Pixel0);
							Pixel1 = _mm_min_ps(Mul, Pixel1);
							Pixel2 = _mm_min_ps(Mul, Pixel2);
							Pixel3 = _mm_min_ps(Mul, Pixel3);

							// Convert float to 32-bit integer
							// values are <=255 so no overflow; NaNs and too-large negative values
							// convert to INT_MIN, all saturate to 0 during pack (no special clamp necessary)
							__m128i Pixel0i = _mm_cvttps_epi32(Pixel0);
							__m128i Pixel1i = _mm_cvttps_epi32(Pixel1);
							__m128i Pixel2i = _mm_cvttps_epi32(Pixel2);
							__m128i Pixel3i = _mm_cvttps_epi32(Pixel3);

							// pack to 8-bit components
							__m128i Out0 = _mm_packs_epi32(Pixel0i, Pixel1i);
							__m128i Out1 = _mm_packs_epi32(Pixel2i, Pixel3i);
							__m128i Out = _mm_packus_epi16(Out0, Out1);

							// store 4xBGRA8
							_mm_storeu_si128((__m128i*)&DestColors[TexelIndex], Out);
#else
							check(false); // not supported for other platforms, see ParallelLoop
#endif
						},
						[DestColors, SrcColors](int64 TexelIndex)
						{
							DestColors[TexelIndex] = SrcColors[TexelIndex].QuantizeRound();
						}
					);
				}
			}
			break;
		
		case ERawImageFormat::BGRE8:
			{
				FColor* DestColors = (FColor*) DestImage.RawData;
				ParallelLoop(TEXT("Texture.CopyImage.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcColors](int64 TexelIndex)
				{
					DestColors[TexelIndex] = SrcColors[TexelIndex].ToRGBE();
				});
			}
			break;
		
		case ERawImageFormat::RGBA16:
			{
				uint16* DestColors = (uint16 *) DestImage.RawData;
				ParallelLoop(TEXT("Texture.CopyImage.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcColors](int64 TexelIndex)
				{
					FLinearColor Src = SrcColors[TexelIndex];
					uint16* Dst = DestColors + TexelIndex * 4;
					Dst[0] = FColor::QuantizeUNormFloatTo16(Src.R);
					Dst[1] = FColor::QuantizeUNormFloatTo16(Src.G);
					Dst[2] = FColor::QuantizeUNormFloatTo16(Src.B);
					Dst[3] = FColor::QuantizeUNormFloatTo16(Src.A);
				});
			}
			break;
		
		case ERawImageFormat::RGBA16F:
			{
				FFloat16Color* DestColors = (FFloat16Color*) DestImage.RawData;
				ParallelLoop(TEXT("Texture.CopyImage.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcColors](int64 TexelIndex)
				{
					DestColors[TexelIndex] = FFloat16Color(SrcColors[TexelIndex]);
				});
			}
			break;

		case ERawImageFormat::R16F:
			{
				FFloat16* DestColors = (FFloat16*) DestImage.RawData;
				ParallelLoop(TEXT("Texture.CopyImage.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcColors](int64 TexelIndex)
				{
					DestColors[TexelIndex] = FFloat16(SrcColors[TexelIndex].R);
				});
			}
			break;
			
		case ERawImageFormat::R32F:
			{
				float* DestColors = (float*) DestImage.RawData;
				ParallelLoop(TEXT("Texture.CopyImage.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcColors](int64 TexelIndex)
				{
					DestColors[TexelIndex] = SrcColors[TexelIndex].R;
				});
			}
			break;

		default:
			check(0);
			break;
		}
	}
	else if (DestImage.Format == ERawImageFormat::RGBA32F)
	{
		// Convert to 32-bit linear floating point.
		FLinearColor* DestColors = (FLinearColor*) DestImage.RawData;
		switch (SrcImage.Format)
		{
		case ERawImageFormat::G8:
			{
				const uint8* SrcLum = (const uint8*) SrcImage.RawData;
				switch (SrcImage.GammaSpace)
				{
				case EGammaSpace::Linear:
					ParallelLoop(TEXT("Texture.CopyImage.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcLum](int64 TexelIndex)
					{
						uint8 Lum = SrcLum[TexelIndex];
						FColor SrcColor(Lum, Lum, Lum, 255);
						DestColors[TexelIndex] = SrcColor.ReinterpretAsLinear();
					});
					break;
				case EGammaSpace::sRGB:
					ParallelLoop(TEXT("Texture.CopyImage.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcLum](int64 TexelIndex)
					{
							uint8 Lum = SrcLum[TexelIndex];
							FColor SrcColor(Lum, Lum, Lum, 255);
							DestColors[TexelIndex] = FLinearColor(SrcColor);
					});
					break;
				case EGammaSpace::Pow22:
					ParallelLoop(TEXT("Texture.CopyImage.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcLum](int64 TexelIndex)
					{
						uint8 Lum = SrcLum[TexelIndex];
						FColor SrcColor(Lum, Lum, Lum, 255);
						DestColors[TexelIndex] = FLinearColor::FromPow22Color(SrcColor);
					});
					break;
				}
			}
			break;

		case ERawImageFormat::G16:
			{
				const uint16* SrcLum = (const uint16*) SrcImage.RawData;
				ParallelLoop(TEXT("Texture.CopyImage.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcLum](int64 TexelIndex)
				{
					float Value = FColor::DequantizeUNorm16ToFloat(SrcLum[TexelIndex]);
					DestColors[TexelIndex] = FLinearColor(Value, Value, Value, 1.0f);
				});
			}
			break;

		case ERawImageFormat::BGRA8:
			{
				const FColor* SrcColors = (const FColor*) SrcImage.RawData;
				switch (SrcImage.GammaSpace)
				{
				case EGammaSpace::Linear:
					ParallelLoop<4>(TEXT("Texture.CopyImage.PF"), NumJobs, TexelsPerJob, NumTexels,
						[DestColors, SrcColors](int64 TexelIndex)
						{
#if PLATFORM_CPU_X86_FAMILY
							// load 4x BGRA8
							__m128i Pixels = _mm_loadu_si128((const __m128i*)&SrcColors[TexelIndex]);

							// expand 8-bit components to 32-bit
							__m128i Zero = _mm_setzero_si128();
							__m128i PixelsL = _mm_unpacklo_epi8(Pixels, Zero);
							__m128i PixelsH = _mm_unpackhi_epi8(Pixels, Zero);
							__m128i Pixel0 = _mm_unpacklo_epi16(PixelsL, Zero);
							__m128i Pixel1 = _mm_unpackhi_epi16(PixelsL, Zero);
							__m128i Pixel2 = _mm_unpacklo_epi16(PixelsH, Zero);
							__m128i Pixel3 = _mm_unpackhi_epi16(PixelsH, Zero);

							// scale to [0,1]
							__m128 OneOver255 = _mm_set_ps1(1.f / 255.f);
							__m128 Out0 = _mm_mul_ps(_mm_cvtepi32_ps(Pixel0), OneOver255);
							__m128 Out1 = _mm_mul_ps(_mm_cvtepi32_ps(Pixel1), OneOver255);
							__m128 Out2 = _mm_mul_ps(_mm_cvtepi32_ps(Pixel2), OneOver255);
							__m128 Out3 = _mm_mul_ps(_mm_cvtepi32_ps(Pixel3), OneOver255);

							// BGRA -> RGBA
							Out0 = _mm_shuffle_ps(Out0, Out0, _MM_SHUFFLE(3, 0, 1, 2));
							Out1 = _mm_shuffle_ps(Out1, Out1, _MM_SHUFFLE(3, 0, 1, 2));
							Out2 = _mm_shuffle_ps(Out2, Out2, _MM_SHUFFLE(3, 0, 1, 2));
							Out3 = _mm_shuffle_ps(Out3, Out3, _MM_SHUFFLE(3, 0, 1, 2));

							// Store 4 pixels
							_mm_storeu_ps(&DestColors[TexelIndex + 0].Component(0), Out0);
							_mm_storeu_ps(&DestColors[TexelIndex + 1].Component(0), Out1);
							_mm_storeu_ps(&DestColors[TexelIndex + 2].Component(0), Out2);
							_mm_storeu_ps(&DestColors[TexelIndex + 3].Component(0), Out3);
#else
							check(false); // not supported for other platforms, see ParallelLoop
#endif
						},
						[DestColors, SrcColors](int64 TexelIndex)
						{
							DestColors[TexelIndex] = SrcColors[TexelIndex].ReinterpretAsLinear();
						}
					);
					break;
				case EGammaSpace::sRGB:
					ParallelLoop(TEXT("Texture.CopyImage.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcColors](int64 TexelIndex)
					{
						DestColors[TexelIndex] = FLinearColor(SrcColors[TexelIndex]);
					});
					break;
				case EGammaSpace::Pow22:
					ParallelLoop(TEXT("Texture.CopyImage.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcColors](int64 TexelIndex)
					{
						DestColors[TexelIndex] = FLinearColor::FromPow22Color(SrcColors[TexelIndex]);
					});
					break;
				}
			}
			break;

		case ERawImageFormat::BGRE8:
			{
				const FColor* SrcColors = (const FColor*) SrcImage.RawData;
				ParallelLoop(TEXT("Texture.CopyImage.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcColors](int64 TexelIndex)
				{
					DestColors[TexelIndex] = SrcColors[TexelIndex].FromRGBE();
				});
			}
			break;

		case ERawImageFormat::RGBA16:
			{
				const uint16* SrcColors = (const uint16*) SrcImage.RawData;
				ParallelLoop(TEXT("Texture.CopyImage.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcColors](int64 TexelIndex)
				{
					int64 SrcIndex = TexelIndex * 4;
					DestColors[TexelIndex] = FLinearColor(
						SrcColors[SrcIndex + 0] * (1.f/65535.0f),
						SrcColors[SrcIndex + 1] * (1.f/65535.0f),
						SrcColors[SrcIndex + 2] * (1.f/65535.0f),
						SrcColors[SrcIndex + 3] * (1.f/65535.0f)
					);
				});
			}
			break;

		case ERawImageFormat::RGBA16F:
			{
				const FFloat16Color* SrcColors = (const FFloat16Color*) SrcImage.RawData;
				ParallelLoop(TEXT("Texture.CopyImage.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcColors](int64 TexelIndex)
				{
					DestColors[TexelIndex] = SrcColors[TexelIndex].GetFloats();
				});
			}
			break;

		case ERawImageFormat::R16F:
			{
				const FFloat16* SrcColors = (const FFloat16*) SrcImage.RawData;
				ParallelLoop(TEXT("Texture.CopyImage.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcColors](int64 TexelIndex)
				{
					DestColors[TexelIndex] = FLinearColor(SrcColors[TexelIndex].GetFloat(), 0, 0, 1);
				});
			}
			break;
			
		case ERawImageFormat::R32F:
			{
				const float* SrcColors = (const float*) SrcImage.RawData;
				ParallelLoop(TEXT("Texture.CopyImage.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcColors](int64 TexelIndex)
				{
					DestColors[TexelIndex] = FLinearColor(SrcColors[TexelIndex], 0, 0, 1);
				});
			}
			break;

		default:
			check(0);
			break;
		}
	}
	else if (SrcImage.Format == ERawImageFormat::BGRA8 && DestImage.Format == ERawImageFormat::RGBA16 &&
		SrcImage.GammaSpace == EGammaSpace::Linear )
	{
		const FColor* SrcColors = (const FColor*) SrcImage.RawData;
		uint16* DestColors = (uint16*) DestImage.RawData;
		ParallelLoop<4>(TEXT("Texture.CopyImage.PF"), NumJobs, TexelsPerJob, NumTexels,
			[DestColors, SrcColors](int64 TexelIndex)
			{
#if PLATFORM_CPU_X86_FAMILY
				// load 4x BGRA
				__m128i Pixels = _mm_loadu_si128((const __m128i*)&SrcColors[TexelIndex]);

				// 8-bit to 16-bit conversion (same as multiply with 65535.f/255.f)
				__m128i Pixel01 = _mm_unpacklo_epi8(Pixels, Pixels);
				__m128i Pixel23 = _mm_unpackhi_epi8(Pixels, Pixels);

				// BGRA -> RGBA
				Pixel01 = _mm_shufflelo_epi16(Pixel01, _MM_SHUFFLE(3, 0, 1, 2));
				Pixel01 = _mm_shufflehi_epi16(Pixel01, _MM_SHUFFLE(3, 0, 1, 2));
				Pixel23 = _mm_shufflelo_epi16(Pixel23, _MM_SHUFFLE(3, 0, 1, 2));
				Pixel23 = _mm_shufflehi_epi16(Pixel23, _MM_SHUFFLE(3, 0, 1, 2));

				// store two 2xRGBA16 pixels
				_mm_storeu_si128((__m128i*)&DestColors[TexelIndex * 4 + 0], Pixel01);
				_mm_storeu_si128((__m128i*)&DestColors[TexelIndex * 4 + 8], Pixel23);
#else
				check(false); // not supported for other platforms, see ParallelLoop
#endif
			},
			[DestColors, SrcColors](int64 TexelIndex)
			{
				FColor Src = SrcColors[TexelIndex];
				uint16* Dst = DestColors + TexelIndex * 4;
				Dst[0] = (Src.R << 8) | Src.R;
				Dst[1] = (Src.G << 8) | Src.G;
				Dst[2] = (Src.B << 8) | Src.B;
				Dst[3] = (Src.A << 8) | Src.A;
			}
		);
	}
	else if (SrcImage.Format == ERawImageFormat::RGBA16 && DestImage.Format == ERawImageFormat::BGRA8 &&
		DestImage.GammaSpace == EGammaSpace::Linear )
	{
		const uint16 * SrcChannels = (const uint16*) SrcImage.RawData;
		FColor* DestColors = (FColor*) DestImage.RawData;
		ParallelLoop(TEXT("Texture.CopyImage.PF"), NumJobs, TexelsPerJob, NumTexels,
			[DestColors, SrcChannels](int64 TexelIndex)
			{
				const uint16* Src = SrcChannels + TexelIndex * 4;
				DestColors[TexelIndex] = FColor(
					Requantize16to8(Src[0]),
					Requantize16to8(Src[1]),
					Requantize16to8(Src[2]),
					Requantize16to8(Src[3]));
			}
		);
	}
	else if (SrcImage.Format == ERawImageFormat::BGRA8 && DestImage.Format == ERawImageFormat::G8 &&
		SrcImage.GammaSpace == DestImage.GammaSpace )
	{
		const FColor* SrcColors = (const FColor*) SrcImage.RawData;
		uint8* DestLum = (uint8*) DestImage.RawData;
		ParallelLoop(TEXT("Texture.CopyImage.PF"), NumJobs, TexelsPerJob, NumTexels,
			[DestLum, SrcColors](int64 TexelIndex)
			{
				DestLum[TexelIndex] = SrcColors[TexelIndex].R;
			}
		);
	}
	else if (SrcImage.Format == ERawImageFormat::G8 && DestImage.Format == ERawImageFormat::BGRA8 &&
		SrcImage.GammaSpace == DestImage.GammaSpace )
	{
		const uint8* SrcLum = (const uint8*) SrcImage.RawData;
		FColor* DestColors = (FColor*) DestImage.RawData;
		ParallelLoop(TEXT("Texture.CopyImage.PF"), NumJobs, TexelsPerJob, NumTexels,
			[SrcLum, DestColors](int64 TexelIndex)
			{
				uint8 Lum = SrcLum[TexelIndex];
				DestColors[TexelIndex] = FColor(Lum,Lum,Lum,255);
			}
		);
	}
	else
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Texture.CopyImage.TempLinear);

		// Arbitrary conversion, use 32-bit linear float as an intermediate format.
		// this is unnecessarily expensive to do something like G8 to R16F, but rare
		// if this shows up as a hot spot, identify the formats using this path and add direct conversions between them

		if ( SrcImage.GetNumPixels()*16 < 65536 ) // RGBA32F fits in 64K
		{
			// whole image temp

			FImage TempImage(SrcImage.SizeX, SrcImage.SizeY, SrcImage.NumSlices, ERawImageFormat::RGBA32F, EGammaSpace::Linear);
			FImageCore::CopyImage(SrcImage, TempImage);
			FImageCore::CopyImage(TempImage, DestImage);
		}
		else
		{
			// use per-line temp, not whole image temp!

			ImageParallelFor(TEXT("PF.CopyImage.TempLinear"),SrcImage,[&](const FImageView & SrcImagePart,int64 StartY)
			{
				FImage TempRow(SrcImage.SizeX, 1, 1, ERawImageFormat::RGBA32F, EGammaSpace::Linear);
				int64 NumRows = SrcImagePart.SizeY;

				for(int64 RowY = StartY; RowY < StartY+NumRows; RowY++)
				{
					FImageView SrcRow  = ImageParallelForGetOneRowView(SrcImage,RowY);
					FImageView DestRow = ImageParallelForGetOneRowView(DestImage,RowY);
				
					FImageCore::CopyImage(SrcRow, TempRow);
					FImageCore::CopyImage(TempRow,DestRow);
				}
			});
		}
	}
}

FORCEINLINE static FLinearColor SaturateToHalfFloat(const FLinearColor& LinearCol)
{
	static constexpr float MAX_HALF_FLOAT16 = 65504.0f;

	FLinearColor Result;
	Result.R = FMath::Clamp(LinearCol.R, -MAX_HALF_FLOAT16, MAX_HALF_FLOAT16);
	Result.G = FMath::Clamp(LinearCol.G, -MAX_HALF_FLOAT16, MAX_HALF_FLOAT16);
	Result.B = FMath::Clamp(LinearCol.B, -MAX_HALF_FLOAT16, MAX_HALF_FLOAT16);
	Result.A = LinearCol.A;
	return Result;
}

void FImage::TransformToWorkingColorSpace(const FVector2d& SourceRedChromaticity, const FVector2d& SourceGreenChromaticity, const FVector2d& SourceBlueChromaticity, const FVector2d& SourceWhiteChromaticity, UE::Color::EChromaticAdaptationMethod Method, double EqualityTolerance)
{
	return FImageCore::TransformToWorkingColorSpace(*this, SourceRedChromaticity, SourceGreenChromaticity, SourceBlueChromaticity, SourceWhiteChromaticity, Method, EqualityTolerance);
}

/* FImage constructors
 *****************************************************************************/
 
FImage::FImage(int32 InSizeX, int32 InSizeY, int32 InNumSlices, ERawImageFormat::Type InFormat, EGammaSpace InGammaSpace)
	: FImageInfo(InSizeX,InSizeY,InNumSlices,InFormat,InGammaSpace)
{
	InitImageStorage(*this);
}

void FImage::Init(const FImageInfo & Info)
{
	// assign the FImageInfo part of this :
	FImageInfo * MyInfo = this; // implicit cast
	*MyInfo = Info;
	InitImageStorage(*this);
}

void FImage::Init(int32 InSizeX, int32 InSizeY, int32 InNumSlices, ERawImageFormat::Type InFormat, EGammaSpace InGammaSpace)
{
	SizeX = InSizeX;
	SizeY = InSizeY;
	NumSlices = InNumSlices;
	Format = InFormat;
	GammaSpace = InGammaSpace;
	InitImageStorage(*this);
}


void FImage::Init(int32 InSizeX, int32 InSizeY, ERawImageFormat::Type InFormat, EGammaSpace InGammaSpace)
{
	SizeX = InSizeX;
	SizeY = InSizeY;
	NumSlices = 1;
	Format = InFormat;
	GammaSpace = InGammaSpace;
	InitImageStorage(*this);
}


/* FImage interface
 *****************************************************************************/
 
void FImage::FreeData(bool bAsyncDetached)
{
	if ( !bAsyncDetached )
	{
		RawData.Empty();
	}
	else
	{
		// do the free on a task, without waiting for it (detached)
		// but do immediately invalidate the RawData member

		TArray64<uint8>* PtrImageData = new TArray64<uint8> ();
		::Swap(RawData,*PtrImageData);
			
		UE::Tasks::Launch(TEXT("FImage.FreeData.Task"),
			[PtrImageData]() // by value, not ref
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FImage.FreeData.Task);
				delete PtrImageData;
				return true;
			},
			LowLevelTasks::ETaskPriority::BackgroundHigh);

		// beware this can cause false out-of-memory conditions
		//	because the next time an alloc tries to find memory, there may still be pending free tasks
		//	to fix that, use "robust detached frees"
	}
}

void FImage::ChangeFormat(ERawImageFormat::Type DestFormat, EGammaSpace DestGammaSpace)
{
	if ( Format == DestFormat &&
		( GammaSpace == DestGammaSpace || ! ERawImageFormat::GetFormatNeedsGammaSpace(Format)) )
	{
		// no action needed
	}
	else
	{
		FImage Temp;
		CopyTo(Temp,DestFormat,DestGammaSpace);
		Swap(Temp);
	}
}

void FImage::CopyTo(FImage& DestImage, ERawImageFormat::Type DestFormat, EGammaSpace DestGammaSpace) const
{
	// if gamma correction is done, it's always *TO* sRGB , not to Pow22
	// so if Pow22 was requested, change to sRGB
	// so that Float->int->Float roundtrips correctly
	if ( DestGammaSpace == EGammaSpace::Pow22 && GammaSpace != EGammaSpace::Pow22 )
	{
		// fix call sites that hit this
		UE_LOGF(LogImageCore, Warning, "Pow22 should not be used as a Dest GammaSpace.  Pow22 Source should encode to sRGB Dest.");
		DestGammaSpace = EGammaSpace::sRGB;
	}

	// existing contents of DestImage are freed and replaced
	DestImage.Init(SizeX,SizeY,NumSlices,DestFormat,DestGammaSpace);
	FImageCore::CopyImage(*this, DestImage);
}

void FImageView::CopyTo(FImage& DestImage, ERawImageFormat::Type DestFormat, EGammaSpace DestGammaSpace) const
{
	// if gamma correction is done, it's always *TO* sRGB , not to Pow22
	// so if Pow22 was requested, change to sRGB
	// so that Float->int->Float roundtrips correctly
	if ( DestGammaSpace == EGammaSpace::Pow22 && GammaSpace != EGammaSpace::Pow22 )
	{
		// fix call sites that hit this
		UE_LOGF(LogImageCore, Warning, "Pow22 should not be used as a Dest GammaSpace.  Pow22 Source should encode to sRGB Dest.");
		DestGammaSpace = EGammaSpace::sRGB;
	}

	DestImage.Init(SizeX,SizeY,NumSlices,DestFormat,DestGammaSpace);
	FImageCore::CopyImage(*this, DestImage);
}

// legacy API, prefer ResizeImage
void FImage::ResizeTo(FImage& DestImage, int32 DestSizeX, int32 DestSizeY, ERawImageFormat::Type DestFormat, EGammaSpace DestGammaSpace) const
{
	FImageCore::ResizeTo(*this,DestImage,DestSizeX,DestSizeY,DestFormat,DestGammaSpace);
}

// legacy API, prefer ResizeImage
void FImageCore::ResizeTo(const FImageView & SourceImage,FImage& DestImage, int32 DestSizeX, int32 DestSizeY, ERawImageFormat::Type DestFormat, EGammaSpace DestGammaSpace)
{
	FImageCore::ResizeImageAllocDest(SourceImage,DestImage,DestSizeX,DestSizeY,DestFormat,DestGammaSpace);
}

IMAGECORE_API FImageView FImageView::GetSlice(int32 SliceIndex) const
{
	check( SliceIndex >= 0 && SliceIndex < NumSlices );
	FImageView Ret = *this;
	Ret.NumSlices = 1;
	int64 BytesPerSlice = Ret.GetImageSizeBytes();
	Ret.RawData = (void *)((uint8 *)RawData + SliceIndex * BytesPerSlice);
	return Ret;
}	

IMAGECORE_API FImageView FImage::GetSlice(int32 SliceIndex) const
{
	check( SliceIndex >= 0 && SliceIndex < NumSlices );
	FImageView Ret = *this;
	Ret.NumSlices = 1;
	int64 BytesPerSlice = Ret.GetImageSizeBytes();
	check( RawData.Num() == NumSlices * BytesPerSlice );
	Ret.RawData = (void *)(RawData.GetData() + SliceIndex * BytesPerSlice);
	return Ret;
}	

void FImage::Linearize(uint8 SourceEncoding, FImage& DestImage) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.Linearize);

	DestImage.SizeX = SizeX;
	DestImage.SizeY = SizeY;
	DestImage.NumSlices = NumSlices;
	DestImage.Format = ERawImageFormat::RGBA32F;
	DestImage.GammaSpace = EGammaSpace::Linear;
	InitImageStorage(DestImage);

	const FImage& SrcImage = *this;
	check(SrcImage.SizeX == DestImage.SizeX);
	check(SrcImage.SizeY == DestImage.SizeY);
	check(SrcImage.NumSlices == DestImage.NumSlices);

	//NOTE: SrcImage has GammaSpace in addition to SourceEncoding
	// if SourceEncoding == None, the SrcImage Gamma is used
	// else SourceEncoding overrides SrcImage Gamma
	// note that SrcImage Gamma applies only on U8 formats (GetFormatNeedsGammaSpace)
	// but SourceEncoding applies on ALL formats (eg. float->float)

	UE::Color::EEncoding SourceEncodingType = static_cast<UE::Color::EEncoding>(SourceEncoding);

	if (SourceEncodingType == UE::Color::EEncoding::None)
	{
		// note SrcImage.GammaSpace is decoded by CopyImage, is that intended?
		FImageCore::CopyImage(SrcImage, DestImage);
		return;
	}
	else if (SourceEncodingType >= UE::Color::EEncoding::Max)
	{
		UE_LOGF(LogImageCore, Warning, "Invalid encoding %d, falling back to linearization using CopyImage.", SourceEncoding);
		FImageCore::CopyImage(SrcImage, DestImage);
		return;
	}

	// fast path common case of sRGB decoding, just use CopyImage, it's much faster
	// note that CopyImage only does Gamma on 8-bit formats (GetFormatNeedsGammaSpace)
	//	but Linearize does it on ALL formats
	if (SourceEncodingType == UE::Color::EEncoding::sRGB &&
		ERawImageFormat::GetFormatNeedsGammaSpace(SrcImage.Format) )
	{
		FImageView SrcImageView(SrcImage);
		SrcImageView.GammaSpace = EGammaSpace::sRGB;
		FImageCore::CopyImage(SrcImageView, DestImage);
		return;
	}
	else if (SourceEncodingType == UE::Color::EEncoding::Gamma22 &&
		ERawImageFormat::GetFormatNeedsGammaSpace(SrcImage.Format) )
	{
		FImageView SrcImageView(SrcImage);
		SrcImageView.GammaSpace = EGammaSpace::Pow22;
		FImageCore::CopyImage(SrcImageView, DestImage);
		return;
	}
	else if (SourceEncodingType == UE::Color::EEncoding::Linear )
	{
		FImageView SrcImageView(SrcImage);
		SrcImageView.GammaSpace = EGammaSpace::Linear;
		FImageCore::CopyImage(SrcImageView, DestImage);
		return;
	}

	// slow case
	// this function calls a func pointer per pixel

	const int64 NumTexels = int64(SrcImage.SizeX) * SrcImage.SizeY * SrcImage.NumSlices;

	int64 TexelsPerJob;
	int32 NumJobs = ParallelForComputeNumJobs(TexelsPerJob,NumTexels,MinPixelsPerJob,MinPixelsForAnyJob);

	TFunction<FLinearColor(const FLinearColor&)> DecodeFunction = UE::Color::GetColorDecodeFunction(SourceEncodingType);
	check(DecodeFunction != nullptr);

	// Convert to 32-bit linear floating point.
	FLinearColor* DestColors = DestImage.AsRGBA32F().GetData();
	switch (SrcImage.Format)
	{
		case ERawImageFormat::G8:
		{
			const uint8* SrcLum = SrcImage.AsG8().GetData();
			ParallelLoop(TEXT("Texture.Linearize.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcLum, DecodeFunction](int64 TexelIndex)
			{
				FColor SrcColor(SrcLum[TexelIndex], SrcLum[TexelIndex], SrcLum[TexelIndex], 255);
				FLinearColor Color = SrcColor.ReinterpretAsLinear();
				Color = DecodeFunction(Color);
				DestColors[TexelIndex] = SaturateToHalfFloat(Color);
			});
			break;
		}

		case ERawImageFormat::G16:
		{
			const uint16* SrcLum = SrcImage.AsG16().GetData();
			ParallelLoop(TEXT("Texture.Linearize.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcLum, DecodeFunction](int64 TexelIndex)
			{
				float Value = FColor::DequantizeUNorm16ToFloat(SrcLum[TexelIndex]);
				FLinearColor Color = FLinearColor(Value, Value, Value, 1.0f);
				Color = DecodeFunction(Color);
				DestColors[TexelIndex] = SaturateToHalfFloat(Color);
			});
			break;
		}

		case ERawImageFormat::BGRA8:
		{
			const FColor* SrcColors = SrcImage.AsBGRA8().GetData();
			ParallelLoop(TEXT("Texture.Linearize.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcColors, DecodeFunction](int64 TexelIndex)
			{
				FLinearColor Color = SrcColors[TexelIndex].ReinterpretAsLinear();
				Color = DecodeFunction(Color);
				DestColors[TexelIndex] = SaturateToHalfFloat(Color);
			});
			break;
		}

		case ERawImageFormat::BGRE8:
		{
			const FColor* SrcColors = SrcImage.AsBGRE8().GetData();
			ParallelLoop(TEXT("Texture.Linearize.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcColors, DecodeFunction](int64 TexelIndex)
			{
				FLinearColor Color = SrcColors[TexelIndex].FromRGBE();
				Color = DecodeFunction(Color);
				DestColors[TexelIndex] = SaturateToHalfFloat(Color);
			});
			break;
		}

		case ERawImageFormat::RGBA16:
		{
			const uint16* SrcColors = SrcImage.AsRGBA16().GetData();
			ParallelLoop(TEXT("Texture.Linearize.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcColors, DecodeFunction](int64 TexelIndex)
			{
				int64 SrcIndex = TexelIndex * 4;
				FLinearColor Color(
					SrcColors[SrcIndex + 0] * (1.f/65535.0f),
					SrcColors[SrcIndex + 1] * (1.f/65535.0f),
					SrcColors[SrcIndex + 2] * (1.f/65535.0f),
					SrcColors[SrcIndex + 3] * (1.f/65535.0f)
				);
				Color = DecodeFunction(Color);
				DestColors[TexelIndex] = SaturateToHalfFloat(Color);
			});
			break;
		}

		case ERawImageFormat::RGBA16F:
		{
			const FFloat16Color* SrcColors = SrcImage.AsRGBA16F().GetData();
			ParallelLoop(TEXT("Texture.Linearize.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcColors, DecodeFunction](int64 TexelIndex)
			{
				FLinearColor Color = SrcColors[TexelIndex].GetFloats();
				Color = DecodeFunction(Color);
				DestColors[TexelIndex] = SaturateToHalfFloat(Color);
			});
			break;
		}

		case ERawImageFormat::RGBA32F:
		{
			const FLinearColor* SrcColors = SrcImage.AsRGBA32F().GetData();
			ParallelLoop(TEXT("Texture.Linearize.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcColors, DecodeFunction](int64 TexelIndex)
			{
				FLinearColor Color = DecodeFunction(SrcColors[TexelIndex]);
				DestColors[TexelIndex] = SaturateToHalfFloat(Color);
			});
			break;
		}

		case ERawImageFormat::R16F:
		{
			const FFloat16* SrcColors = SrcImage.AsR16F().GetData();
			ParallelLoop(TEXT("Texture.Linearize.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcColors, DecodeFunction](int64 TexelIndex)
			{
				FLinearColor Color(SrcColors[TexelIndex].GetFloat(), 0, 0, 1);
				Color = DecodeFunction(Color);
				DestColors[TexelIndex] = SaturateToHalfFloat(Color);
			});
			break;
		}
		
		case ERawImageFormat::R32F:
		{
			const float* SrcColors = SrcImage.AsR32F().GetData();
			ParallelLoop(TEXT("Texture.Linearize.PF"), NumJobs, TexelsPerJob, NumTexels, [DestColors, SrcColors, DecodeFunction](int64 TexelIndex)
			{
				FLinearColor Color(SrcColors[TexelIndex], 0, 0, 1);
				Color = DecodeFunction(Color);
				DestColors[TexelIndex] = SaturateToHalfFloat(Color);
			});
			break;
		}
	}
}



void FMipMapImage::Init(ERawImageFormat::Type InFormat, EGammaSpace InGammaSpace)
{
	Format = InFormat;
	GammaSpace = InGammaSpace;
	SubImages.Empty();
	RawData.Empty();
}

void FMipMapImage::Init(int32 MipZeroWidth, int32 MipZeroHeight, int32 NumMips, ERawImageFormat::Type InFormat, EGammaSpace InGammaSpace)
{
	Format = InFormat;
	GammaSpace = InGammaSpace;

	const int32 BytesPerPixel = ERawImageFormat::GetBytesPerPixel(Format);
	int64 BufferSizeBytes = 0;

	int32 CurrMipWidth = MipZeroWidth;
	int32 CurrMipHeight = MipZeroHeight;

	SubImages.Empty(NumMips);
	
	do 
	{
		const int32 MipSizeBytes = CurrMipWidth * CurrMipHeight * BytesPerPixel;
		SubImages.Add(FMipInfo{ CurrMipWidth, CurrMipHeight, BufferSizeBytes, MipSizeBytes });

		BufferSizeBytes += MipSizeBytes;
		CurrMipWidth = CurrMipWidth >> 1;
		CurrMipHeight = CurrMipHeight >> 1;
		NumMips--;
	} while ((NumMips > 0) && (CurrMipWidth > 0) && (CurrMipHeight > 0));

	RawData.Empty(BufferSizeBytes);
	RawData.AddUninitialized(BufferSizeBytes);
}

void FMipMapImage::CopyTo(FMipMapImage& DestImage, ERawImageFormat::Type DestFormat, EGammaSpace DestGammaSpace)
{
	// if gamma correction is done, it's always *TO* sRGB , not to Pow22
	// so if Pow22 was requested, change to sRGB
	// so that Float->int->Float roundtrips correctly
	if (DestGammaSpace == EGammaSpace::Pow22 && GammaSpace != EGammaSpace::Pow22)
	{
		// fix call sites that hit this
		UE_LOGF(LogImageCore, Warning, "Pow22 should not be used as a Dest GammaSpace.  Pow22 Source should encode to sRGB Dest.");
		DestGammaSpace = EGammaSpace::sRGB;
	}

	constexpr int32 MipLevel = 0;
	int32 Width, Height;
	if (GetMipDimensions(MipLevel, Width, Height))
	{
		// existing contents of DestImage are freed and replaced
		DestImage.Init(Width, Height, GetMipCount(), DestFormat, DestGammaSpace);
		ParallelFor(GetMipCount(), [this, &DestImage](int32 MipLevel) 
		{
			FImageView SrcMipImageView = GetMipImage(MipLevel);
			FImageView DestMipImageView = DestImage.GetMipImage(MipLevel);
			FImageCore::CopyImage(SrcMipImageView, DestMipImageView);
		});
	}
}

void FMipMapImage::ChangeFormat(ERawImageFormat::Type DestFormat, EGammaSpace DestGammaSpace)
{
	if (Format == DestFormat &&
		(GammaSpace == DestGammaSpace || !ERawImageFormat::GetFormatNeedsGammaSpace(Format)))
	{
		// no action needed
	}
	else
	{
		FMipMapImage Temp;
		CopyTo(Temp, DestFormat, DestGammaSpace);
		::Swap(*this, Temp);
	}
}

const FUtf8StringView ERawImageFormat::GetNameView(Type Format)
{
	switch (Format)
	{
	case ERawImageFormat::G8:  return UTF8TEXTVIEW("G8");
	case ERawImageFormat::G16: return UTF8TEXTVIEW("G16");
	case ERawImageFormat::R16F: return UTF8TEXTVIEW("R16F");
	case ERawImageFormat::R32F: return UTF8TEXTVIEW("R32F");
	case ERawImageFormat::BGRA8: return UTF8TEXTVIEW("BGRA8");
	case ERawImageFormat::BGRE8: return UTF8TEXTVIEW("BGRE8");
	case ERawImageFormat::RGBA16: return UTF8TEXTVIEW("RGBA16");
	case ERawImageFormat::RGBA16F: return UTF8TEXTVIEW("RGBA16F");
	case ERawImageFormat::RGBA32F: return UTF8TEXTVIEW("RGBA32F");
	default:
		check(0);
		return UTF8TEXTVIEW("invalid");
	}
}


IMAGECORE_API const TCHAR * ERawImageFormat::GetName(Type Format)
{
	switch (Format)
	{
	case ERawImageFormat::G8:  return TEXT("G8");
	case ERawImageFormat::G16: return TEXT("G16");
	case ERawImageFormat::R16F: return TEXT("R16F");
	case ERawImageFormat::R32F: return TEXT("R32F");
	case ERawImageFormat::BGRA8: return TEXT("BGRA8");
	case ERawImageFormat::BGRE8: return TEXT("BGRE8");
	case ERawImageFormat::RGBA16: return TEXT("RGBA16");
	case ERawImageFormat::RGBA16F: return TEXT("RGBA16F");
	case ERawImageFormat::RGBA32F: return TEXT("RGBA32F");
	default:
		check(0);
		return TEXT("invalid");
	}
}

bool ERawImageFormat::GetFormatFromString(FUtf8StringView InString, ERawImageFormat::Type& OutType)
{
	if (InString.Compare(UTF8TEXTVIEW("G8"), ESearchCase::IgnoreCase) == 0) { OutType = ERawImageFormat::G8; return true; }
	else if (InString.Compare(UTF8TEXTVIEW("G16"), ESearchCase::IgnoreCase) == 0) { OutType = ERawImageFormat::G16; return true; }
	else if (InString.Compare(UTF8TEXTVIEW("R16F"), ESearchCase::IgnoreCase) == 0) { OutType = ERawImageFormat::R16F; return true; }
	else if (InString.Compare(UTF8TEXTVIEW("R32F"), ESearchCase::IgnoreCase) == 0) { OutType = ERawImageFormat::R32F; return true; }
	else if (InString.Compare(UTF8TEXTVIEW("BGRA8"), ESearchCase::IgnoreCase) == 0) { OutType = ERawImageFormat::BGRA8; return true; }
	else if (InString.Compare(UTF8TEXTVIEW("BGRE8"), ESearchCase::IgnoreCase) == 0) { OutType = ERawImageFormat::BGRE8; return true; }
	else if (InString.Compare(UTF8TEXTVIEW("RGBA16"), ESearchCase::IgnoreCase) == 0) { OutType = ERawImageFormat::RGBA16; return true; }
	else if (InString.Compare(UTF8TEXTVIEW("RGBA16F"), ESearchCase::IgnoreCase) == 0) { OutType = ERawImageFormat::RGBA16F; return true; }
	else if (InString.Compare(UTF8TEXTVIEW("RGBA32F"), ESearchCase::IgnoreCase) == 0) { OutType = ERawImageFormat::RGBA32F; return true; }
	else if (InString.Compare(UTF8TEXTVIEW("Invalid"), ESearchCase::IgnoreCase) == 0) { OutType = ERawImageFormat::Invalid; return true; }

	// We distinguish between "they stored invalid on purpose" and "we didn't recognize it" by the return value.
	OutType = ERawImageFormat::Invalid;
	return false;
}

IMAGECORE_API int64 ERawImageFormat::GetBytesPerPixel(Type Format)
{
	int64 BytesPerPixel = 0;
	switch (Format)
	{
	case ERawImageFormat::G8:
		BytesPerPixel = 1;
		break;
	
	case ERawImageFormat::G16:
	case ERawImageFormat::R16F:
		BytesPerPixel = 2;
		break;
		
	case ERawImageFormat::R32F:
	case ERawImageFormat::BGRA8:
	case ERawImageFormat::BGRE8:
		BytesPerPixel = 4;
		break;
			
	case ERawImageFormat::RGBA16:
	case ERawImageFormat::RGBA16F:
		BytesPerPixel = 8;
		break;

	case ERawImageFormat::RGBA32F:
		BytesPerPixel = 16;
		break;

	default:
		check(0);
		break;
	}
	return BytesPerPixel;
}


IMAGECORE_API int ERawImageFormat::NumChannels(Type Format)
{
	switch(Format)
	{
	case ERawImageFormat::G8: return 1;
	case ERawImageFormat::BGRA8: return 4;
	case ERawImageFormat::BGRE8: return 3; // <-
	case ERawImageFormat::RGBA16: return 4;
	case ERawImageFormat::RGBA16F: return 4;
	case ERawImageFormat::RGBA32F: return 4;
	case ERawImageFormat::G16: return 1;
	case ERawImageFormat::R16F: return 1;
	case ERawImageFormat::R32F: return 1;
	default:
		check(0);
		return 0;
	}
}

IMAGECORE_API bool ERawImageFormat::IsU8Channels(Type Format)
{
	// "BGRE8" is 8-bit but not *U8*
	return Format == G8 || Format == BGRA8;
}

IMAGECORE_API bool ERawImageFormat::IsHDR(Type Format)
{
	return Format == RGBA16F || Format == RGBA32F || Format == R16F || Format == R32F || Format == BGRE8;
}

IMAGECORE_API bool ERawImageFormat::HasAlphaChannel(Type Format)
{
	return Format == BGRA8 || Format == RGBA16 || Format == RGBA16F || Format == RGBA32F;
}

IMAGECORE_API const FLinearColor ERawImageFormat::GetOnePixelLinear(const void * PixelData,ERawImageFormat::Type Format,EGammaSpace Gamma)
{
	switch(Format)
	{
	case G8:
	{
		uint8 Gray = ((const uint8 *)PixelData)[0];
		FColor Color(Gray,Gray,Gray);
		if ( Gamma == EGammaSpace::sRGB )
			return FLinearColor::FromSRGBColor(Color);
		else if ( Gamma == EGammaSpace::Pow22 )
			return FLinearColor::FromPow22Color(Color);
		else
			return Color.ReinterpretAsLinear();
	}
	case G16:
	{
		const uint16 * Samples = (const uint16 *)PixelData;
		float Gray = Samples[0]*(1.f/65535.f);
		return FLinearColor(Gray,Gray,Gray,1.f);
	}
	case BGRA8:
	{
		FColor Color = *((const FColor *)PixelData);
		if ( Gamma == EGammaSpace::sRGB )
			return FLinearColor::FromSRGBColor(Color);
		else if ( Gamma == EGammaSpace::Pow22 )
			return FLinearColor::FromPow22Color(Color);
		else
			return Color.ReinterpretAsLinear();
	}
	case BGRE8:
	{
		FColor Color = *((const FColor *)PixelData);
		return Color.FromRGBE();
	}
	case RGBA16:
	{
		const uint16 * Samples = (const uint16 *)PixelData;
		return FLinearColor(
			Samples[0]*(1.f/65535.f), 
			Samples[1]*(1.f/65535.f), 
			Samples[2]*(1.f/65535.f), 
			Samples[3]*(1.f/65535.f));
	}
	case RGBA16F:
	{
		FFloat16Color Color = *((const FFloat16Color *)PixelData);
		return Color.GetFloats();
	}
	case RGBA32F:
	{
		return *((const FLinearColor *)PixelData);
	}
	case R16F:
	{
		float R = FPlatformMath::LoadHalf((const uint16 *)PixelData);
		return FLinearColor(R,0,0,1.f);
	}
	case R32F:
	{
		float R = *((const float *)PixelData);
		return FLinearColor(R,0,0,1.f);
	}
	
	default:
		check(0);
		return FLinearColor();
	}
}


void FImageCore::SanitizeFloat16AndSetAlphaOpaqueForBC6H(const FImageView & InOutImageWhole)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.SanitizeFloat16AndSetAlphaOpaqueForBC6H);
	check(InOutImageWhole.Format == ERawImageFormat::RGBA16F);

	ImageParallelFor(TEXT("PF.SanitizeFloat16AndSetAlphaOpaqueForBC6H"),InOutImageWhole,[](const FImageView & InOutImagePart,int64 RowY)
	{
		const int64 TexelNum = InOutImagePart.GetNumPixels();
		FFloat16Color* Data = reinterpret_cast<FFloat16Color*>(InOutImagePart.RawData);

		for (int64 TexelIndex = 0; TexelIndex < TexelNum; ++TexelIndex)
		{
			FFloat16Color& F16Color = Data[TexelIndex];

			F16Color.R = F16Color.R.GetClampedNonNegativeAndFinite();
			F16Color.G = F16Color.G.GetClampedNonNegativeAndFinite();
			F16Color.B = F16Color.B.GetClampedNonNegativeAndFinite();
			F16Color.A.SetOne();
		}
	} );
}

bool FImageCore::DetectAlphaChannel(const FImageView & InImage)
{
	if ( ! HasAlphaChannel(InImage.Format) )
	{
		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.DetectAlphaChannel);

	// opaque alpha threshold where we'd quantize to < 255 in U8
	//  images with only alpha larger than this are treated as opaque
	const float FloatNonOpaqueAlpha = 254.5f / 255.f; // the U8 alpha threshold

	int64 NumTexels = InImage.GetNumPixels();
	int64 TexelsPerJob;
	int32 NumJobs = ImageParallelForComputeNumJobsForPixels(TexelsPerJob,NumTexels);

	if (InImage.Format == ERawImageFormat::BGRA8)
	{
		TArrayView64<const FColor> SrcColorArray = InImage.AsBGRA8();
		check(SrcColorArray.Num() == NumTexels);

		const FColor* ColorPtr = &SrcColorArray[0];
		
		return ParallelOr(TEXT("Texture.DetectAlphaChannel.PF"), NumJobs, TexelsPerJob, NumTexels, [&](int64 TexelIndex)
		{
			return (ColorPtr[TexelIndex].A != 255);
		});
	}
	else if (InImage.Format == ERawImageFormat::RGBA32F)
	{
		TArrayView64<const FLinearColor> SrcColorArray = InImage.AsRGBA32F();
		check(SrcColorArray.Num() == NumTexels);

		const FLinearColor* ColorPtr = &SrcColorArray[0];
		
		return ParallelOr(TEXT("Texture.DetectAlphaChannel.PF"), NumJobs, TexelsPerJob, NumTexels, [&](int64 TexelIndex)
		{
			return (ColorPtr[TexelIndex].A <= FloatNonOpaqueAlpha);
		});
	}
	else if (InImage.Format == ERawImageFormat::RGBA16)
	{
		TArrayView64<const uint16> SrcChannelArray = InImage.AsRGBA16();
		check(SrcChannelArray.Num() == NumTexels * 4);

		const uint16* ChannelPtr = &SrcChannelArray[0];
		
		return ParallelOr(TEXT("Texture.DetectAlphaChannel.PF"), NumJobs, TexelsPerJob, NumTexels, [&](int64 TexelIndex)
		{
			return (ChannelPtr[TexelIndex*4+3] != 0xFFFF);
		});
	}
	else if (InImage.Format == ERawImageFormat::RGBA16F)
	{
		TArrayView64<const FFloat16Color> SrcColorArray = InImage.AsRGBA16F();
		check(SrcColorArray.Num() == NumTexels);

		const FFloat16Color* ColorPtr = &SrcColorArray[0];
		
		// 16F closest to 1.0 is 0.99951172
		// use the float tolerance here? or check exactly ?
		// use the same FloatNonOpaqueAlpha tolerance for consistency ?

		return ParallelOr(TEXT("Texture.DetectAlphaChannel.PF"), NumJobs, TexelsPerJob, NumTexels, [&](int64 TexelIndex)
		{
			return (ColorPtr[TexelIndex].A.GetFloat() <= FloatNonOpaqueAlpha);
		});
	}
	else
	{
		// new format ?
		//	formats without HasAlphaChannel were previously excluded
		check(0);
	}

	return false;
}

static void SetAlphaOpaque_SingleThreaded(const FImageView & InImage)
{
	int64 NumPixels = (int64)InImage.SizeX * InImage.SizeY * InImage.NumSlices;

	if (InImage.Format == ERawImageFormat::BGRA8)
	{
		TArrayView64<FColor> SrcColorArray = InImage.AsBGRA8();
		check(SrcColorArray.Num() == NumPixels);

		FColor* ColorPtr = &SrcColorArray[0];
		FColor* EndPtr = ColorPtr + SrcColorArray.Num();

		for (; ColorPtr < EndPtr; ++ColorPtr)
		{
			ColorPtr->A = 255;
		}
	}
	else if (InImage.Format == ERawImageFormat::RGBA32F)
	{
		TArrayView64<FLinearColor> SrcColorArray = InImage.AsRGBA32F();
		check(SrcColorArray.Num() == NumPixels);

		FLinearColor* ColorPtr = &SrcColorArray[0];
		FLinearColor* EndPtr = ColorPtr + SrcColorArray.Num();

		for (; ColorPtr < EndPtr; ++ColorPtr)
		{
			ColorPtr->A = 1.f;
		}
	}
	else if (InImage.Format == ERawImageFormat::RGBA16)
	{
		TArrayView64<uint16> SrcChannelArray = InImage.AsRGBA16();
		check(SrcChannelArray.Num() == NumPixels * 4);

		uint16* ChannelPtr = &SrcChannelArray[0];
		uint16* EndPtr = ChannelPtr + SrcChannelArray.Num();

		for (; ChannelPtr < EndPtr; ChannelPtr += 4)
		{
			ChannelPtr[3] = 0xFFFF;
		}
	}
	else if (InImage.Format == ERawImageFormat::RGBA16F)
	{
		TArrayView64<FFloat16Color> SrcColorArray = InImage.AsRGBA16F();
		check(SrcColorArray.Num() == NumPixels);

		FFloat16Color* ColorPtr = &SrcColorArray[0];
		FFloat16Color* EndPtr = ColorPtr + SrcColorArray.Num();

		FFloat16 One(1.f);

		for (; ColorPtr < EndPtr; ++ColorPtr)
		{
			ColorPtr->A = One;
		}
	}
	else if (InImage.Format == ERawImageFormat::G8 ||
		InImage.Format == ERawImageFormat::BGRE8 ||
		InImage.Format == ERawImageFormat::G16 ||
		InImage.Format == ERawImageFormat::R16F ||
		InImage.Format == ERawImageFormat::R32F)
	{
		// source image formats don't have alpha
	}
	else
	{
		// new format ?
		check(0);
	}
}

void FImageCore::SetAlphaOpaque(const FImageView & InImage)
{
	if ( ! HasAlphaChannel(InImage.Format) )
	{
		return;
	}

	ImageParallelFor(TEXT("PF.SetAlphaOpaque"),InImage,[](const FImageView & Part,int64 Row) {
		SetAlphaOpaque_SingleThreaded(Part);
	} );
}

void FImageCore::TransformToWorkingColorSpace(const FImageView& InLinearImage, const FVector2d& SourceRedChromaticity, const FVector2d& SourceGreenChromaticity, const FVector2d& SourceBlueChromaticity, const FVector2d& SourceWhiteChromaticity, UE::Color::EChromaticAdaptationMethod Method, double EqualityTolerance)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.TransformToWorkingColorSpace);

	check(InLinearImage.GammaSpace == EGammaSpace::Linear);

	const UE::Color::FColorSpace Source(SourceRedChromaticity, SourceGreenChromaticity, SourceBlueChromaticity, SourceWhiteChromaticity);
	const UE::Color::FColorSpace& Target = UE::Color::FColorSpace::GetWorking();

	if (Source.Equals(Target, EqualityTolerance))
	{
		UE_LOGF(LogImageCore, VeryVerbose, "Source and working color spaces are equal within tolerance, bypass color space transformation.");
		return;
	}

	UE::Color::FColorSpaceTransform Transform(Source, Target, Method);

	FLinearColor* ImageColors = InLinearImage.AsRGBA32F().GetData();

	const int64 NumTexels = int64(InLinearImage.SizeX) * InLinearImage.SizeY * InLinearImage.NumSlices;
	int64 TexelsPerJob;
	int32 NumJobs = ImageParallelForComputeNumJobsForPixels(TexelsPerJob, NumTexels);

	ParallelLoop(TEXT("Texture.TransformToWorkingColorSpace.PF"), NumJobs, TexelsPerJob, NumTexels, [Transform, ImageColors](int64 TexelIndex)
		{
			FLinearColor Color = ImageColors[TexelIndex];
			Color = Transform.Apply(Color);
			ImageColors[TexelIndex] = SaturateToHalfFloat(Color);
		});
}

static constexpr uint32 ImageInfoSerializationVersion = 1;

void FImageInfo::ImageInfoToCompactBinary(FCbObject& OutObject) const
{
	FCbWriter Writer;
	Writer.BeginObject();
	Writer.AddInteger("Version", ImageInfoSerializationVersion);
	Writer.AddInteger("SizeX", SizeX);
	Writer.AddInteger("SizeY", SizeY);
	Writer.AddInteger("NumSlices", NumSlices);
	Writer.AddInteger("GammaSpace", (uint8)GammaSpace);
	Writer.AddString("Format", ERawImageFormat::GetNameView(Format));
	Writer.EndObject();
	OutObject = Writer.Save().AsObject();
}

bool FImageInfo::ImageInfoFromCompactBinary(const FCbObject& InObject)
{
	if (InObject["Version"].AsUInt32() != ImageInfoSerializationVersion)
	{
		return false;
	}

	SizeX = InObject["SizeX"].AsInt32();
	SizeY = InObject["SizeY"].AsInt32();
	NumSlices = InObject["NumSlices"].AsInt32();
	GammaSpace = (EGammaSpace)InObject["GammaSpace"].AsUInt8();
	if (ERawImageFormat::GetFormatFromString(InObject["Format"].AsString(), Format) == false)
	{
		return false;
	}

	return true;
}



//--------------------------------------------------
//{ resize

#if USING_CODE_ANALYSIS
	// disable static analysis warnings from inside stb_image_resize2.h
	MSVC_PRAGMA( warning(push) )
	MSVC_PRAGMA( warning( disable : ALL_CODE_ANALYSIS_WARNINGS ) )
#endif

THIRD_PARTY_INCLUDES_START
#define STBIR_ASSERT(x) checkSlow(x)
#define STBIR_MALLOC(size,user_data) FMemory::Malloc(size,16)
#define STBIR_FREE(ptr,user_data)	 FMemory::Free(ptr)
#define STB_IMAGE_RESIZE_IMPLEMENTATION
#define STBIR_DONT_CHANGE_FP_CONTRACT
#include <stb_image_resize2.h>
THIRD_PARTY_INCLUDES_END

#if USING_CODE_ANALYSIS
	MSVC_PRAGMA( warning(pop) )
#endif

namespace FImageCore
{

static bool GetFormatSTBIR(ERawImageFormat::Type Format,EGammaSpace GammaSpace,
	int & DestNumChannels,
	stbir_datatype & DestDataType,
	stbir_pixel_layout & DestLayout)
{
	check( Format < ERawImageFormat::MAX ); // MAX is not max, it is count

	// the "PM" Layouts mean STBIR doesn't change us to pre-multiplied
	//	we are not pre-multiplied
	//	the RGB channels are filtered without reference to the alpha values

	switch(Format)
	{
		case ERawImageFormat::G8: // G8 = gray = replicate to RGB
			DestNumChannels = 1;
			if ( GammaSpace != EGammaSpace::Linear ) // Pow22 also
				DestDataType = STBIR_TYPE_UINT8_SRGB;
			else
				DestDataType = STBIR_TYPE_UINT8;
			DestLayout = STBIR_1CHANNEL;
			return true;
		case ERawImageFormat::BGRA8:
			DestNumChannels = 4;
			if ( GammaSpace != EGammaSpace::Linear ) // Pow22 also
				DestDataType = STBIR_TYPE_UINT8_SRGB; // A is linear
			else
				DestDataType = STBIR_TYPE_UINT8;
			DestLayout = STBIR_BGRA_PM;
			return true;
		case ERawImageFormat::BGRE8:
			return false; // BGRE cannot use STB
		case ERawImageFormat::RGBA16:
			DestNumChannels = 4;
			DestDataType = STBIR_TYPE_UINT16;
			DestLayout = STBIR_RGBA_PM;
			return true;
		case ERawImageFormat::RGBA16F:
			DestNumChannels = 4;
			DestDataType = STBIR_TYPE_HALF_FLOAT;
			DestLayout = STBIR_RGBA_PM;
			return true;
		case ERawImageFormat::RGBA32F:
			DestNumChannels = 4;
			DestDataType = STBIR_TYPE_FLOAT;
			DestLayout = STBIR_RGBA_PM;
			return true;
		case ERawImageFormat::G16: // G16 = gray = replicate to RGB
			DestNumChannels = 1;
			DestDataType = STBIR_TYPE_UINT16;
			DestLayout = STBIR_1CHANNEL;
			return true;
		case ERawImageFormat::R16F:
			DestNumChannels = 1;
			DestDataType = STBIR_TYPE_HALF_FLOAT;
			DestLayout = STBIR_1CHANNEL;
			return true;
		case ERawImageFormat::R32F:
			DestNumChannels = 1;
			DestDataType = STBIR_TYPE_FLOAT;
			DestLayout = STBIR_1CHANNEL;
			return true;
			
		default: // Invalid!
			return false;
	}
}

static stbir_pixel_layout ChangeToAlphaWeighted(stbir_pixel_layout layout)
{
	// incoming layout is not alpha weighted

	// we initially made 4-channel RGBA formats with the _PM flag
	// to tell STBIR (falsely) that they are pre-multiplied already
	// that makes STBIR do non-alpha-weighted resize, which is our default
	// instead now take off the _PM flag
	// that tells STBIR to do alpha-weighted resize

	switch(layout)
	{
	case STBIR_RGBA_PM: return STBIR_RGBA;
	case STBIR_BGRA_PM: return STBIR_BGRA;
	default:
		check(0); // we're only called on 4-channel layouts
		return layout;
	}
}

static bool FilterIsNopWhenSameSize(EResizeImageFilter FilterWithFlags)
{
	EResizeImageFilter Filter = FilterWithFlags & EResizeImageFilter::WithoutFlagsMask;
	switch(Filter)
	{
		case EResizeImageFilter::PointSample:
		case EResizeImageFilter::Box:
		case EResizeImageFilter::Triangle:
		case EResizeImageFilter::Default: // see stbir__set_sampler
		case EResizeImageFilter::AdaptiveSharp:
		case EResizeImageFilter::AdaptiveSmooth:
			return true;
		
		// Cubic filters still change the image with size the same
		case EResizeImageFilter::CubicGaussian:
		case EResizeImageFilter::CubicSharp:
		case EResizeImageFilter::CubicMitchell:
		case EResizeImageFilter::MitchellOneQuarter:
		case EResizeImageFilter::MitchellOneSixth:
		case EResizeImageFilter::MitchellNegOneSixth:
		case EResizeImageFilter::MitchellNegOneThird:
		case EResizeImageFilter::Lanczos4:
		case EResizeImageFilter::Lanczos5:
			return false;

		default: // all enum values should be explicitly listed above
			check(false);
			return false;
	}
}

// B in [0,5/3]
//	smaller B = sharper, larger B = blurrier
//  B = 0 is the catmull-rom interpolating b-spline
//	B = 1/3 is standard compromise Mitchell
//  B = 1 is the cubic Gaussian
// in between pulse_cubic_mitchell_B = lerp( catrom, cubic, B )
static float pulse_cubic_mitchell_B(const float signed_x,const float B)
{
	const float ax = fabsf(signed_x);
	if (ax<1.f)
	{
		return ax*ax* ( ((9/6.f)-(6/6.f)*B)*ax - (15/6.f) + (9/6.f)*B ) + ((6/6.f) - (2/6.f)*B);
	}
	else if (ax<2.f)
	{
		return ax*ax* ( ((2/6.f)*B - (3/6.f))*ax + ((15/6.f) - (9/6.f)*B) ) + ((12/6.f)*B - (24/6.f))*ax + ((12/6.f) - (4/6.f)*B);
	}
	else
	{
		return 0.0f;
	}
}

// template on integer denominator to make Mitchell filters of various B :
//	(note denom can be negative but not zero)
template <int denom> 
static float my_filter_mitchell_B(float x, float s, void * user_data)
{
	const float B = 1.f/denom;

	return pulse_cubic_mitchell_B(x,B);
}

static float my_pulse_sinc(const float x) 
{
	const float pix = (3.141592653590f) * x;

	const float tiny = 1e-4f;
	if ( fabsf(pix) < tiny )
	{
		// Taylor series of sin for x small
		return 1.f - (pix*pix)*(1/6.f);
	}
	else
	{
		return sinf(pix) / pix;
	}
}

// "support" must match the support function used
static float my_filter_lanczos(float x, float support)
{
	if ( fabsf(x) >= support ) return 0.f; // ? is this necessary, or does STBIR do this already ?

	float pulse = my_pulse_sinc(x);
	float window = my_pulse_sinc(x * (1.f/support));
	return pulse * window;
}

static float my_filter_lanczos4(float x, float s, void * user_data)
{
	return my_filter_lanczos(x,2.f);
}
static float my_filter_lanczos5(float x, float s, void * user_data)
{	
	return my_filter_lanczos(x,2.5f);
}

static float my_filter_support_two(float s, void * user_data) 
{
	return 2;
}
static float my_filter_support_twoandhalf(float s, void * user_data) 
{
	return 2.5f;
}

struct stbir_kernel_and_support
{
	stbir__kernel_callback * kernel = nullptr;
	stbir__support_callback * support = nullptr;
};

static const stbir_kernel_and_support c_my_ks_mitchell_onequarter = { my_filter_mitchell_B<4> , my_filter_support_two };
static const stbir_kernel_and_support c_my_ks_mitchell_onesixth = { my_filter_mitchell_B<6> , my_filter_support_two };
static const stbir_kernel_and_support c_my_ks_mitchell_negonesixth = { my_filter_mitchell_B<-6> , my_filter_support_two };
static const stbir_kernel_and_support c_my_ks_mitchell_negonethird = { my_filter_mitchell_B<-3> , my_filter_support_two };
static const stbir_kernel_and_support c_my_ks_lanczos4 = { my_filter_lanczos4 , my_filter_support_two };
static const stbir_kernel_and_support c_my_ks_lanczos5 = { my_filter_lanczos5 , my_filter_support_twoandhalf };


static stbir_filter MapFilterToStb(EResizeImageFilter FilterWithFlags,int64 SizeFm,int64 SizeTo,uint32 WrapFlag, stbir_edge & OutStbirEdgeMode,
	stbir_kernel_and_support & OutCustomKS)
{
	if ( SizeFm == SizeTo && FilterIsNopWhenSameSize(FilterWithFlags) )
	{
		OutStbirEdgeMode = STBIR_EDGE_CLAMP;
		return STBIR_FILTER_POINT_SAMPLE;
	}
	
	OutStbirEdgeMode = WrapFlag ? STBIR_EDGE_WRAP : STBIR_EDGE_CLAMP;
	
	EResizeImageFilter Filter = FilterWithFlags & EResizeImageFilter::WithoutFlagsMask;
	
	//const stbir_filter STBIR_FILTER_OutCustomKS = STBIR_FILTER_OTHER; // <- not allowed
	const stbir_filter STBIR_FILTER_OutCustomKS = STBIR_FILTER_DEFAULT;

	switch(Filter)
	{
		case EResizeImageFilter::Box: return STBIR_FILTER_BOX;
		case EResizeImageFilter::Triangle: return STBIR_FILTER_TRIANGLE;
		case EResizeImageFilter::CubicGaussian: return STBIR_FILTER_CUBICBSPLINE;
		case EResizeImageFilter::CubicSharp: return STBIR_FILTER_CATMULLROM;
		case EResizeImageFilter::CubicMitchell: return STBIR_FILTER_MITCHELL;
		case EResizeImageFilter::PointSample: return STBIR_FILTER_POINT_SAMPLE;
		
		case EResizeImageFilter::MitchellOneQuarter:
			OutCustomKS = c_my_ks_mitchell_onequarter;
			return STBIR_FILTER_OutCustomKS;
		case EResizeImageFilter::MitchellOneSixth:
			OutCustomKS = c_my_ks_mitchell_onesixth;
			return STBIR_FILTER_OutCustomKS;
		case EResizeImageFilter::MitchellNegOneSixth:
			OutCustomKS = c_my_ks_mitchell_negonesixth;
			return STBIR_FILTER_OutCustomKS;
		case EResizeImageFilter::MitchellNegOneThird:
			OutCustomKS = c_my_ks_mitchell_negonethird;
			return STBIR_FILTER_OutCustomKS;
		case EResizeImageFilter::Lanczos4:
			OutCustomKS = c_my_ks_lanczos4;
			return STBIR_FILTER_OutCustomKS;
		case EResizeImageFilter::Lanczos5:
			OutCustomKS = c_my_ks_lanczos5;
			return STBIR_FILTER_OutCustomKS;

		case EResizeImageFilter::Default: // AdaptiveSharp matches STBIR_FILTER_DEFAULT
		case EResizeImageFilter::AdaptiveSharp:
			return ( SizeFm < SizeTo ) ? STBIR_FILTER_CATMULLROM : STBIR_FILTER_MITCHELL;
			
		case EResizeImageFilter::AdaptiveSmooth:
			return ( SizeFm < SizeTo ) ? STBIR_FILTER_MITCHELL : STBIR_FILTER_CUBICBSPLINE;

		default:
			check(false);
			return STBIR_FILTER_DEFAULT;
	}
}
	
}; // namespace

IMAGECORE_API void FImageCore::ResizeImage(const FImageView & SourceImage,const FImageView & DestImage, EResizeImageFilter Filter)
{
	check( SourceImage.IsImageInfoValid() );
	check( DestImage.IsImageInfoValid() );

	int64 DestNumPixels = DestImage.GetNumPixels();
	if ( DestNumPixels == 0 )
	{
		return;
	}

	if ( SourceImage.NumSlices != 1 || DestImage.NumSlices != 1 )
	{
		if ( SourceImage.NumSlices == DestImage.NumSlices )
		{
			// resize slices one by one (no filtering across slices)
			//	eg. for arrays, not a 3d filter for cubes

			ParallelFor(TEXT("ResizeImage.Slices"), SourceImage.NumSlices, 1, [&](int64 SliceIndex)
			{
				const FImageView SourceSlice = SourceImage.GetSlice(SliceIndex);
				const FImageView DestSlice = DestImage.GetSlice(SliceIndex);
				ResizeImage(SourceSlice,DestSlice,Filter);
			}, EParallelForFlags::Unbalanced);
		}
		else
		{
			// different slice counts

			UE_LOGF(LogImageCore,Error,"ResizeImage: different slice count : %d -> %d",
				SourceImage.NumSlices,DestImage.NumSlices);

			// zero out DestImage bytes?
		}

		return;
	}

	// single slice resize
	check( SourceImage.NumSlices == 1 && DestImage.NumSlices == 1 );

	if ( SourceImage.SizeX == DestImage.SizeX && SourceImage.SizeY == DestImage.SizeY &&
		FilterIsNopWhenSameSize(Filter) )
	{
		// images are same size, just blit
		//	note manually selected Cubic filters run on same-size image will still apply the filter
		FImageCore::CopyImage(SourceImage,DestImage);
		return;
	}
	
	// can't write Pow22 :
	check( DestImage.GetGammaSpace() != EGammaSpace::Pow22 );

	int SourceNumChannels;
	stbir_datatype SourceDataType;
	stbir_pixel_layout SourceLayout;
	bool SourceOk = GetFormatSTBIR(SourceImage.Format,SourceImage.GetGammaSpace(),
		SourceNumChannels,SourceDataType,SourceLayout);
		
	int DestNumChannels;
	stbir_datatype DestDataType;
	stbir_pixel_layout DestLayout;
	bool DestOk = GetFormatSTBIR(DestImage.Format,DestImage.GetGammaSpace(),
		DestNumChannels,DestDataType,DestLayout);

	if ( !SourceOk || !DestOk || SourceNumChannels != DestNumChannels )
	{
		// Source and Dest formats have different channel count
		//	it's hard to get this right in all cases through the stb_resize API
		//	so just convert through linear
	
		FImageView SourceImageF32Linear = SourceImage;
		FImage SourceScratch;
		if ( SourceImageF32Linear.Format != ERawImageFormat::RGBA32F )
		{
			// convert source to linear, alloc scratch
			SourceImage.CopyTo(SourceScratch,ERawImageFormat::RGBA32F,EGammaSpace::Linear);
			SourceImageF32Linear = SourceScratch;
		}
		
		FImageView DestImageF32Linear = DestImage;
		FImage DestScratch;
		if ( DestImageF32Linear.Format != ERawImageFormat::RGBA32F )
		{
			// alloc empty linear DestScratch
			DestScratch.Init(DestImage.SizeX,DestImage.SizeY,ERawImageFormat::RGBA32F,EGammaSpace::Linear);
			DestImageF32Linear = DestScratch;
		}

		// do the resize on the linears :
		ResizeImage(SourceImageF32Linear,DestImageF32Linear, Filter);

		// if we didn't resize directly to dest, copy now :
		if ( DestImageF32Linear.RawData != DestImage.RawData )
		{
			FImageCore::CopyImage(DestImageF32Linear,DestImage);
		}

		return;
	}

	// do the resize!
		
	STBIR_RESIZE resize;
	stbir_resize_init( &resize,
		SourceImage.RawData,SourceImage.SizeX,SourceImage.SizeY,SourceImage.GetBytesPerPixel()*SourceImage.SizeX,
		DestImage.RawData,DestImage.SizeX,DestImage.SizeY,DestImage.GetBytesPerPixel()*DestImage.SizeX,
		SourceLayout,SourceDataType);
	
	// we only use STBIR with the same channel count in src and dest
	//	so questions about how PM/RGBA translate to 1CHANNEL are moot
	check( SourceNumChannels == DestNumChannels );
	if ( (uint32)( Filter & EResizeImageFilter::Flag_AlphaWeighted ) && SourceNumChannels == 4 )
	{
		SourceLayout = ChangeToAlphaWeighted(SourceLayout);
		DestLayout = ChangeToAlphaWeighted(DestLayout);
	}
	
	stbir_set_pixel_layouts(&resize,SourceLayout,DestLayout);
	stbir_set_datatypes(&resize,SourceDataType,DestDataType);

	stbir_edge StbirEdgeX,StbirEdgeY;
	stbir_kernel_and_support StbirKSX,StbirKSY;
	stbir_filter StbirFilterX = MapFilterToStb(Filter,SourceImage.SizeX,DestImage.SizeX, (uint32)(Filter & EResizeImageFilter::Flag_WrapX), StbirEdgeX, StbirKSX);
	stbir_filter StbirFilterY = MapFilterToStb(Filter,SourceImage.SizeY,DestImage.SizeY, (uint32)(Filter & EResizeImageFilter::Flag_WrapY), StbirEdgeY, StbirKSY);

	stbir_set_edgemodes(&resize, StbirEdgeX, StbirEdgeY);	
	stbir_set_filters(&resize, StbirFilterX, StbirFilterY);
	stbir_set_filter_callbacks(&resize,StbirKSX.kernel,StbirKSX.support,StbirKSY.kernel,StbirKSY.support);

	const int32 NumWorkers = FTaskGraphInterface::Get().GetNumWorkerThreads();

	// 2 jobs per worker, minus 1 for some slack :
	int32 MaxNumSplits = NumWorkers*2 - 1;
	
	// parallel over Dest pixels
	if ( MaxNumSplits * MinPixelsPerJob > DestNumPixels )
	{
		MaxNumSplits = (int)(DestNumPixels/MinPixelsPerJob);
	}

	if ( MaxNumSplits <= 1 )
	{
		// just run synchronous :
		bool ok = !! stbir_resize_extended(&resize);
		check(ok);

		return;
	}

	// threaded :

	int64 NumSplits = stbir_build_samplers_with_splits(&resize,MaxNumSplits);
	check( NumSplits > 0 );

	if ( NumSplits == 1 )
	{
		stbir_resize_extended_split(&resize,0,1);
	}
	else
	{
		ParallelFor(TEXT("ResizeImage"), NumSplits, 1, [&](int64 SplitIndex)
		{
			stbir_resize_extended_split(&resize,SplitIndex,1);
		}, EParallelForFlags::Unbalanced);
	}

	stbir_free_samplers(&resize);
}
	
IMAGECORE_API void FImageCore::ResizeImageAllocDest(const FImageView & SourceImage,FImage & DestImage,int32 DestSizeX, int32 DestSizeY, ERawImageFormat::Type DestFormat, EGammaSpace DestGammaSpace, EResizeImageFilter Filter)
{
	check( DestImage.RawData.Num() == 0 || SourceImage.RawData != DestImage.GetPixelPointer(0,0) ); // must not be resizing onto self
	
	// can't write Pow22 :
	check( DestGammaSpace != EGammaSpace::Pow22 );

	// note that if DestImage was already allocated to the right size, this will not re-allocate it
	DestImage.Init(DestSizeX,DestSizeY,SourceImage.NumSlices,DestFormat,DestGammaSpace);
	ResizeImage(SourceImage,DestImage,Filter);
}

IMAGECORE_API void FImageCore::ResizeImageAllocDest(const FImageView & SourceImage,FImage & DestImage,int32 DestSizeX, int32 DestSizeY, EResizeImageFilter Filter)
{
	EGammaSpace DestGammaSpace = SourceImage.GetGammaSpace();
	
	// can't write Pow22 :
	if ( DestGammaSpace == EGammaSpace::Pow22 )
	{
		DestGammaSpace = EGammaSpace::sRGB;
	}

	ResizeImageAllocDest(SourceImage,DestImage,DestSizeX,DestSizeY,SourceImage.Format,DestGammaSpace,Filter);
}

IMAGECORE_API void FImageCore::ResizeImageInPlace(FImage & Image,int32 DestSizeX, int32 DestSizeY, ERawImageFormat::Type DestFormat, EGammaSpace DestGammaSpace, EResizeImageFilter Filter)
{
	if ( DestSizeX == Image.SizeX && DestSizeY == Image.SizeY && DestFormat == Image.Format && DestGammaSpace == Image.GetGammaSpace() &&
		FilterIsNopWhenSameSize(Filter) )
	{
		// nop!
		return;
	}

	// ResizeImageInPlace of Pow22 will change you to sRGB
	if ( DestGammaSpace == EGammaSpace::Pow22 )
	{
		DestGammaSpace = EGammaSpace::sRGB;
	}

	FImage Source;
	Image.Swap(Source);
	ResizeImageAllocDest(Source,Image,DestSizeX,DestSizeY,DestFormat,DestGammaSpace, Filter);
}

IMAGECORE_API void FImageCore::ResizeImageInPlace(FImage & Image,int32 DestSizeX, int32 DestSizeY, EResizeImageFilter Filter)
{
	ResizeImageInPlace(Image,DestSizeX,DestSizeY,Image.Format,Image.GetGammaSpace(), Filter);
}

//} resize
//----------------------


