// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageCoreBP.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ImageCoreBP)

bool USharedImageConstRefBlueprintFns::IsValid(const FSharedImageConstRefBlueprint& InSharedImage)
{
	return InSharedImage.Reference && InSharedImage.Reference->IsImageInfoValid();
}

FVector2f USharedImageConstRefBlueprintFns::GetSize(const FSharedImageConstRefBlueprint& InSharedImage)
{
	if (InSharedImage.Reference &&
		InSharedImage.Reference->IsImageInfoValid())
	{
		return FVector2f((float)InSharedImage.Reference->GetWidth(), (float)InSharedImage.Reference->GetHeight());
	}
	return FVector2f(-1, -1);
}

int USharedImageConstRefBlueprintFns::GetWidth(const FSharedImageConstRefBlueprint& InSharedImage)
{
	if (InSharedImage.Reference &&
		InSharedImage.Reference->IsImageInfoValid())
	{
		return InSharedImage.Reference->GetWidth();
	}
	return -1;
}

int USharedImageConstRefBlueprintFns::GetHeight(const FSharedImageConstRefBlueprint& InSharedImage)
{
	if (InSharedImage.Reference &&
		InSharedImage.Reference->IsImageInfoValid())
	{
		return InSharedImage.Reference->GetHeight();
	}
	return -1;
}

FLinearColor USharedImageConstRefBlueprintFns::GetPixelLinearColor(const FSharedImageConstRefBlueprint& InSharedImage, int32 InX, int32 InY, bool& bOutValid, FLinearColor InFailureColor)
{
	const FImage* Image = InSharedImage.Reference;

	bOutValid = false;
	if (Image == nullptr || Image->IsImageInfoValid() == false)
	{
		return InFailureColor;
	}

	if (InX < 0 || InY < 0 ||
		InX >= Image->GetWidth() || InY >= Image->GetHeight())
	{
		return InFailureColor;
	}

	// GetOnePixelLinear checks() on a bad format so we validate first
	if (Image->Format >= ERawImageFormat::MAX)
	{
		return InFailureColor;
	}

	bOutValid = true;
	return ERawImageFormat::GetOnePixelLinear(Image->GetPixelPointer(InX, InY), Image->Format, Image->GammaSpace);
}

FVector4f USharedImageConstRefBlueprintFns::GetPixelValue(const FSharedImageConstRefBlueprint& InSharedImage, int32 InX, int32 InY, bool& bOutValid)
{
	const FImage* Image = InSharedImage.Reference;

	bOutValid = false;
	if (Image == nullptr || Image->IsImageInfoValid() == false)
	{
		return FVector4f(0, 0, 0, 0);
	}

	if (InX < 0 || InY < 0 ||
		InX >= Image->GetWidth() || InY >= Image->GetHeight())
	{
		return FVector4f(0, 0, 0, 0);
	}

	switch (Image->Format)
	{
	case ERawImageFormat::G8:
		{
			bOutValid = true;
			uint8 G = *(const uint8*)Image->GetPixelPointer(InX, InY);
			return FVector4f((float)G, (float)G, (float)G, 255.0f);
		}
	case ERawImageFormat::BGRA8:
		{
			bOutValid = true;
			FColor Result = *(const FColor*)Image->GetPixelPointer(InX, InY);
			return FVector4f((float)Result.R, (float)Result.G, (float)Result.B, (float)Result.A);
		}
	case ERawImageFormat::BGRE8:
		{
			bOutValid = true;
			FColor Result = *(const FColor*)Image->GetPixelPointer(InX, InY);
			return Result.FromRGBE();
		}
	case ERawImageFormat::RGBA16:
		{
			bOutValid = true;
			const uint16* Result = (const uint16*)Image->GetPixelPointer(InX, InY);
			return FVector4f((float)Result[0], (float)Result[1], (float)Result[2], (float)Result[3]);
		}
	case ERawImageFormat::RGBA16F:
		{
			bOutValid = true;
			const FFloat16* Result = (const FFloat16*)Image->GetPixelPointer(InX, InY);
			return FVector4f(Result[0].GetFloat(), Result[1].GetFloat(), Result[2].GetFloat(), Result[3].GetFloat());
		}
	case ERawImageFormat::RGBA32F:
		{
			bOutValid = true;
			const float* Result = (const float*)Image->GetPixelPointer(InX, InY);
			return FVector4f(Result[0], Result[1], Result[2], Result[3]);
		}
	case ERawImageFormat::G16:
		{
			bOutValid = true;
			uint16 Result = *(const uint16*)Image->GetPixelPointer(InX, InY);
			return FVector4f((float)Result, (float)Result, (float)Result, 1.0f);
		}
	case ERawImageFormat::R16F:
		{
			bOutValid = true;
			const FFloat16* Result = (const FFloat16*)Image->GetPixelPointer(InX, InY);
			float ResultFloat = Result->GetFloat();
			return FVector4f(ResultFloat, 0.0f, 0.0f, 1.0f);
		}
	case ERawImageFormat::R32F:
		{
			bOutValid = true;
			float Result = *(const float*)Image->GetPixelPointer(InX, InY);
			return FVector4f(Result, 0.0f, 0.0f, 1.0f);
		}
	default:
		{
			return FVector4f(0, 0, 0, 0);
		}
	}
}
