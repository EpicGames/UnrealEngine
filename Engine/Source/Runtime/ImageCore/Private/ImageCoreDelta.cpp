// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageCoreDelta.h"
#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"

static TAutoConsoleVariable<int32> CVarTextureUEDeltaDebugColor(
	TEXT("r.TextureUEDeltaDebugColor"),
	0,
	TEXT("UE Delta Tile Debug Color"),
	ECVF_Default);

// none of these values can change, they affect the file format!

static constexpr int64 MinPixelsPerCut = 32768; // = 128K bytes for BGRA8
// Surfaces of default VT tile size or smaller will not parallelize at all :
static constexpr int64 MinPixelsForAnyCut = 136*136;
//static constexpr int64 CutStrideBytes = 8192;
static constexpr int64 CutStrideBytes = 4096;
static constexpr int64 MaxNumCuts = 512; // <- do not use real worker count

// do not use ParallelForComputeNumJobs ; not using real worker count
static inline int64 ImageCoreDeltaComputeNumCuts(int64 NumItems)
{
	if ( NumItems <= MinPixelsPerCut )
	{
		return 1;
	}
	
	int64 NumCuts = (NumItems / MinPixelsPerCut); // round down

	while( NumCuts > MaxNumCuts )
	{
		NumCuts >>= 1;
	}

	return NumCuts;
}

// do not use ImageParallelForComputeNumJobsForRows , because it looks at worker count
//	we must be machine independent here
static int64 ImageCoreDeltaComputeNumCutsForRows(int64 & OutNumRowsPerCut,int64 SizeX,int64 SizeY)
{
	check( SizeX > 0 && SizeY > 0 );

	int64 NumPixels = SizeX*SizeY;
	int64 NumCuts1 = ImageCoreDeltaComputeNumCuts(NumPixels);
	int64 NumRowsPerCut = (SizeY + NumCuts1-1) / NumCuts1;
	
	// recompute NumCuts :
	int64 NumCuts = (SizeY + NumRowsPerCut-1) / NumRowsPerCut;

	check( NumRowsPerCut*NumCuts >= SizeY );
	check( NumRowsPerCut*(NumCuts-1) < SizeY );

	OutNumRowsPerCut = NumRowsPerCut;
	return NumCuts;
}

IMAGECORE_API void FImageCoreDelta::AddSplitStridedViewsForDelta( TArray64<FImageViewStrided> & OutViews, const FImageView & InView )
{
	// note: this splitting must be the same on all machines
	//	eg. do not use core count

	// the splitting logic is in the file format, it must not change!

	for(int64 SliceIndex=0;SliceIndex<InView.NumSlices;SliceIndex++)
	{
		FImageView SliceView = InView.GetSlice(SliceIndex);

		if ( SliceView.GetNumPixels() <= MinPixelsForAnyCut )
		{
			OutViews.Add( FImageViewStrided(SliceView) );
		}
		else
		{
			int64 StrideBytes = SliceView.GetStrideBytes(); // SliceView is dense so this is the width in bytes

			// we want to cut horizontally so that stride fits in L1
			if ( StrideBytes <= CutStrideBytes )
			{
				// no horizontal cuts

				// do vertical cuts for pixel count :
				int64 NumRowsPerCut;
				int64 NumCuts = ImageCoreDeltaComputeNumCutsForRows(NumRowsPerCut,SliceView.SizeX,SliceView.SizeY);
				for(int64 CutIndex=0;CutIndex<NumCuts;CutIndex++)
				{
					int64 StartY = CutIndex * NumRowsPerCut;
					int64 CutSizeY = FMath::Min(NumRowsPerCut,SliceView.SizeY - StartY);

					OutViews.Add( FImageViewStrided(SliceView).GetPortion(0,SliceView.SizeX,StartY,CutSizeY) );
				}
			}
			else
			{
				// yes horizontal cuts

				int64 NumHorizontalParts = FMath::DivideAndRoundUp(StrideBytes,CutStrideBytes);
				int64 HorizontalPartBytes = (StrideBytes + (NumHorizontalParts/2) ) / NumHorizontalParts; // round div (?)
				check( HorizontalPartBytes > 0 && HorizontalPartBytes <= CutStrideBytes );

				// align to cache line :
				//	(this also aligns to whole pixels)
				HorizontalPartBytes = (HorizontalPartBytes + 63) & (~63);
				int64 HorizontalPartPixels = HorizontalPartBytes / SliceView.GetBytesPerPixel();

				// recompute NumHorizontalParts :
				NumHorizontalParts = FMath::DivideAndRoundUp<int32>(SliceView.SizeX,(int32)HorizontalPartPixels);			

				for(int64 HorizontalIndex=0;HorizontalIndex<NumHorizontalParts;HorizontalIndex++)
				{
					int64 StartX = HorizontalIndex * HorizontalPartPixels;
					int64 StripWidthX = FMath::Min(HorizontalPartPixels,SliceView.SizeX - StartX);
					
					// do vertical cuts for pixel count :
					//	(could factor this out of the loop, but StripWidthX does vary on the last column)
					int64 NumRowsPerCut;
					int64 NumCuts = ImageCoreDeltaComputeNumCutsForRows(NumRowsPerCut,StripWidthX,SliceView.SizeY);

					for(int64 CutIndex=0;CutIndex<NumCuts;CutIndex++)
					{
						int64 StartY = CutIndex * NumRowsPerCut;
						int64 CutSizeY = FMath::Min(NumRowsPerCut,SliceView.SizeY - StartY);

						OutViews.Add( FImageViewStrided(SliceView).GetPortion(StartX,StripWidthX,StartY,CutSizeY) );
					}
				}
			}
		}
	}
}

namespace
{

//  do bias or not?
//	without deinterleave, bias only helps a tiny bit (with deinterleave it is a solid benefit)
//	  the main case for multi-byte deltas is RGBA16
//  -> we could expose this out as an option to the API if it's wanted some day
//		the other big thing would be to offer deinterleaving as well
//		(other possibilities: constant channel elision, LOCO transform)
#define UEDELTA_DO_BIAS 1

#if UEDELTA_DO_BIAS
template <typename t_type> struct Bias;

template <> struct Bias<uint8> { static constexpr uint8 Value = 0; };
template <> struct Bias<uint16> { static constexpr uint16 Value = 0x8080U; };
template <> struct Bias<uint32> { static constexpr uint32 Value = 0x80808080U; };

#if PLATFORM_CPU_X86_FAMILY
static const __m128i c_bias_16 = _mm_set1_epi16(Bias<uint16>::Value);
static const __m128i c_bias_32 = _mm_set1_epi32(Bias<uint32>::Value);
#endif

#else

template <typename t_type> struct Bias { static constexpr t_type Value = 0; };

#endif // UEDELTA_DO_BIAS

// Out = In1 - In2
static inline void ImageCoreDeltaSub16bytes(uint8 * Out,const uint8 * In1,const uint8 * In2)
{
#if PLATFORM_CPU_X86_FAMILY
	__m128i V1 = _mm_loadu_si128((const __m128i*)In1);
	__m128i V2 = _mm_loadu_si128((const __m128i*)In2);
	_mm_storeu_si128((__m128i*)Out, _mm_sub_epi8(V1,V2));
#else
	for(int i=0;i<16;i++)
	{
		Out[i] = In1[i] - In2[i];
	}
#endif
}

// Out = In1 + In2
static inline void ImageCoreDeltaAdd16bytes(uint8 * Out,const uint8 * In1,const uint8 * In2)
{
#if PLATFORM_CPU_X86_FAMILY
	__m128i V1 = _mm_loadu_si128((const __m128i*)In1);
	__m128i V2 = _mm_loadu_si128((const __m128i*)In2);
	_mm_storeu_si128((__m128i*)Out, _mm_add_epi8(V1,V2));
#else
	for(int i=0;i<16;i++)
	{
		Out[i] = In1[i] + In2[i];
	}
#endif
}

// Out = In1 - In2 + bias
static inline void ImageCoreDeltaSub16bytes(uint16 * Out,const uint16 * In1,const uint16 * In2)
{
#if PLATFORM_CPU_X86_FAMILY
	__m128i V1 = _mm_loadu_si128((const __m128i*)In1);
	__m128i V2 = _mm_loadu_si128((const __m128i*)In2);
	_mm_storeu_si128((__m128i*)Out, 
	#if UEDELTA_DO_BIAS
		_mm_add_epi16(_mm_sub_epi16(V1,V2), c_bias_16)
	#else
		_mm_sub_epi16(V1,V2)
	#endif
	);
#else
	for(int i=0;i<8;i++)
	{
		Out[i] = In1[i] - In2[i] + Bias<uint16>::Value;
	}
#endif
}

// Out = In1 + In2 - bias
static inline void ImageCoreDeltaAdd16bytes(uint16 * Out,const uint16 * In1,const uint16 * In2)
{
#if PLATFORM_CPU_X86_FAMILY
	__m128i V1 = _mm_loadu_si128((const __m128i*)In1);
	__m128i V2 = _mm_loadu_si128((const __m128i*)In2);
	_mm_storeu_si128((__m128i*)Out, 
	#if UEDELTA_DO_BIAS
		_mm_sub_epi16(_mm_add_epi16(V1,V2), c_bias_16)
	#else
		_mm_add_epi16(V1,V2)
	#endif
	 );
#else
	for(int i=0;i<8;i++)
	{
		Out[i] = In1[i] + In2[i] - Bias<uint16>::Value;
	}
#endif
}

// Out = In1 - In2 + bias
static inline void ImageCoreDeltaSub16bytes(uint32 * Out,const uint32 * In1,const uint32 * In2)
{
#if PLATFORM_CPU_X86_FAMILY
	__m128i V1 = _mm_loadu_si128((const __m128i*)In1);
	__m128i V2 = _mm_loadu_si128((const __m128i*)In2);
	_mm_storeu_si128((__m128i*)Out, 
	#if UEDELTA_DO_BIAS
		_mm_add_epi32(_mm_sub_epi32(V1,V2), c_bias_32) 
	#else
		_mm_sub_epi32(V1,V2)
	#endif
	);
#else
	for(int i=0;i<4;i++)
	{
		Out[i] = In1[i] - In2[i] + Bias<uint32>::Value;
	}
#endif
}

// Out = In1 + In2 - bias
static inline void ImageCoreDeltaAdd16bytes(uint32 * Out,const uint32 * In1,const uint32 * In2)
{
#if PLATFORM_CPU_X86_FAMILY
	__m128i V1 = _mm_loadu_si128((const __m128i*)In1);
	__m128i V2 = _mm_loadu_si128((const __m128i*)In2);
	_mm_storeu_si128((__m128i*)Out,
	#if UEDELTA_DO_BIAS
		_mm_sub_epi32(_mm_add_epi32(V1,V2), c_bias_32) 
	#else
		_mm_add_epi32(V1,V2)
	#endif
	);
#else
	for(int i=0;i<4;i++)
	{
		Out[i] = In1[i] + In2[i] - Bias<uint32>::Value;
	}
#endif
}


// here NumX is now the number of t_type items (== Width or Width*4)
// fill OutData
// forward : read 2 rows from InImage, write OutData = subtract
// reverse : read 1 row of InImage (containing delta), add to previous row of OutData
// we start at Y=1 , the Y=0 row should have already been copied
template <typename t_type>
static inline void DeltaT(const FImageViewStrided & InImage,uint8 * OutData, const int64 NumX, const bool bForward)
{
	constexpr int64 c_type_bytes = sizeof(t_type);
	constexpr int64 c_num_per_16bytes = 16/c_type_bytes;
	
	// NumX16 is the number of 16 byte units
	int64 NumX16 = NumX & (~(c_num_per_16bytes-1));
	int64 TailBytes = (NumX - NumX16) * c_type_bytes;
	check( TailBytes >= 0 && TailBytes < 16 );
	
	t_type TailIn1[c_num_per_16bytes] = { 0 };
	t_type TailIn2[c_num_per_16bytes] = { 0 };
	t_type TailOut[c_num_per_16bytes] = { 0 };

	if ( bForward )
	{
		// start at Y=1, first row was memcpy'ed
		for(int64 Y=1;Y<InImage.SizeY;Y++)
		{
			const uint8 * InPtr8 = InImage.GetRowPointer(Y);
			const t_type * InPtr = (const t_type *)InPtr8;
			t_type * OutPtr = (t_type *)( OutData + (InPtr8 - (const uint8 *)InImage.RawData) );
			const t_type * InPtrUp = (const t_type *)( InPtr8 - InImage.StrideBytes );

			for(int64 X=0;X<NumX16;X+=c_num_per_16bytes)
			{
				ImageCoreDeltaSub16bytes(OutPtr+X,InPtr+X,InPtrUp+X);
			}

			if ( TailBytes )
			{
				memcpy(TailIn1,InPtr+NumX16,TailBytes);
				memcpy(TailIn2,InPtrUp+NumX16,TailBytes);
				ImageCoreDeltaSub16bytes(TailOut,TailIn1,TailIn2);
				memcpy(OutPtr+NumX16,TailOut,TailBytes);
			}
		}
	}
	else
	{
		// start at Y=1, first row was memcpy'ed
		for(int64 Y=1;Y<InImage.SizeY;Y++)
		{
			const uint8 * InPtr8 = InImage.GetRowPointer(Y);
			const t_type * InPtr = (const t_type *)InPtr8;
			t_type * OutPtr = (t_type *)( OutData + (InPtr8 - (const uint8 *)InImage.RawData) );
			const t_type * OutPtrUp = (const t_type *)( (uint8 *)OutPtr - InImage.StrideBytes );

			for(int64 X=0;X<NumX16;X+=c_num_per_16bytes)
			{
				ImageCoreDeltaAdd16bytes(OutPtr+X,InPtr+X,OutPtrUp+X);
			}
			
			if ( TailBytes )
			{
				memcpy(TailIn1,InPtr+NumX16,TailBytes);
				memcpy(TailIn2,OutPtrUp+NumX16,TailBytes);
				ImageCoreDeltaAdd16bytes(TailOut,TailIn1,TailIn2);
				memcpy(OutPtr+NumX16,TailOut,TailBytes);
			}
		}
	}
}

// here NumX is now the number of samples (not the number of pixels)
static void DeltaImage8(const FImageViewStrided & InImage,uint8 * OutData, int64 NumX, bool bForward)
{
	DeltaT<uint8>(InImage,OutData,NumX,bForward);
}

static void DeltaImage16(const FImageViewStrided & InImage,uint8 * OutData, int64 NumX, bool bForward)
{
	DeltaT<uint16>(InImage,OutData,NumX,bForward);
}

static void DeltaImage32(const FImageViewStrided & InImage,uint8 * OutData, int64 NumX, bool bForward)
{
	DeltaT<uint32>(InImage,OutData,NumX,bForward);
}

}; // namespace

IMAGECORE_API void FImageCoreDelta::DoTransform(const FImageViewStrided & InImage,uint8 * OutData,bool bForward)
{
	// no slices :
	check( InImage.NumSlices == 1 );
	check( InImage.SizeX > 0 && InImage.SizeY > 0 );
	
	int64 NumX = InImage.SizeX;
	int64 WidthBytes = NumX * InImage.GetBytesPerPixel();

	check( InImage.StrideBytes >= WidthBytes );
	
	if ( CVarTextureUEDeltaDebugColor.GetValueOnAnyThread() )
	{
		// debug color each tile

		if ( InImage.Format == ERawImageFormat::BGRA8 )
		{
			FColor DebugColor = FColor::MakeRandomColor();

			for(int64 Y=0;Y<InImage.SizeY;Y++)
			{
				const uint8 * InPtr = InImage.GetRowPointer(Y);
				uint8 * OutPtr = OutData + (InPtr - (const uint8 *)InImage.RawData);
			
				for(int64 X=0;X<NumX;X++) { ((FColor *)OutPtr)[X] = DebugColor; }
			}

			return;
		}
	}

	// row 0 is just copied over so every tile is independent
	//	note this makes the exact tile cutting logic baked into the file format
	memcpy(OutData,InImage.RawData,WidthBytes);
	
	switch(InImage.Format)
	{
	case ERawImageFormat::G8:
		DeltaImage8(InImage,OutData,NumX,bForward);
		break;
	case ERawImageFormat::BGRA8:
	case ERawImageFormat::BGRE8:
		DeltaImage8(InImage,OutData,NumX*4,bForward);
		break;
	case ERawImageFormat::RGBA16:
	case ERawImageFormat::RGBA16F:
		DeltaImage16(InImage,OutData,NumX*4,bForward);
		break;
	case ERawImageFormat::RGBA32F:
		DeltaImage32(InImage,OutData,NumX*4,bForward);
		break;
	case ERawImageFormat::G16:
	case ERawImageFormat::R16F:
		DeltaImage16(InImage,OutData,NumX,bForward);
		break;
	case ERawImageFormat::R32F:
		DeltaImage32(InImage,OutData,NumX,bForward);
		break;
	default:
		checkNoEntry();
		break;
	}
}
