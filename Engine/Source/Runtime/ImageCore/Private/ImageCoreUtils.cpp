// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageCoreUtils.h"

DEFINE_LOG_CATEGORY_STATIC(LogImageCoreUtils, Log, All);

IMAGECORE_API bool FImageCoreUtils::IsImageImportPossible(int64 Width,int64 Height)
{
	if ( Width <= 0 || Height <= 0 )
	{
		return false;
	}

	// each dimension must fit in int32
	if ( Width > INT32_MAX || Height > INT32_MAX )
	{
		return false;
	}

	int64 PixelCount = Width * Height; // multiply is safe because of INT32_MAX check

	// for non-VT limitation is 16384 for W/H , and 2 GB for output surface bytes
	// for VT the limitation is that the built VT must fit in 4 GB

	if ( PixelCount > UINT32_MAX )
	{
		return false;
	}

	// still may fail to import due to stricter constraints later

	return true;
}

IMAGECORE_API int32 FImageCoreUtils::GetMipCountFromDimensions(int32 InSizeX, int32 InSizeY, int32 InVolumeZ, bool bInIsVolume)
{
	uint32 MaxMipDimension = FMath::Max3(InSizeX, InSizeY, bInIsVolume ? InVolumeZ : 1);
	return 1 + FMath::FloorLog2(MaxMipDimension);
}


IMAGECORE_API EPixelFormat FImageCoreUtils::GetPixelFormatForRawImageFormat(ERawImageFormat::Type InFormat, 
							ERawImageFormat::Type * pOutEquivalentFormat)
{
	ERawImageFormat::Type OutEquivalentFormatDummy;
	if ( pOutEquivalentFormat == nullptr )
	{
		pOutEquivalentFormat = &OutEquivalentFormatDummy;
	}
		
	// if *pOutEquivalentFormat != InFormat , then conversion is needed
	*pOutEquivalentFormat = InFormat;

	// do not map to the very closest EPixelFormat
	//	instead map to a close one that is actually usable as Texture

	switch(InFormat)
	{
	case ERawImageFormat::G8:    
		return PF_G8;
	case ERawImageFormat::BGRA8: 
		return PF_B8G8R8A8;
	case ERawImageFormat::BGRE8: 
		*pOutEquivalentFormat = ERawImageFormat::RGBA16F;
		return PF_FloatRGBA;
	case ERawImageFormat::RGBA16:  
		return PF_R16G16B16A16_UNORM;
	case ERawImageFormat::G16: 
		return PF_G16;
	case ERawImageFormat::RGBA16F: 
		return PF_FloatRGBA;
	case ERawImageFormat::RGBA32F: 
		*pOutEquivalentFormat = ERawImageFormat::RGBA16F;
		return PF_FloatRGBA;
	case ERawImageFormat::R16F:	
		return PF_R16F;
	case ERawImageFormat::R32F:	
		return PF_R32_FLOAT; // only if bSupportFilteredFloat32Textures
	
	default:
		check(0);
		return PF_Unknown;
	}
}

	// ETextureSourceFormat and ERawImageFormat::Type are one-to-one :
IMAGECORE_API ERawImageFormat::Type FImageCoreUtils::ConvertToRawImageFormat(ETextureSourceFormat Format)
{
	switch(Format)
	{
	case TSF_G8: return ERawImageFormat::G8;
	case TSF_BGRA8: return ERawImageFormat::BGRA8;
	case TSF_BGRE8: return ERawImageFormat::BGRE8;
	case TSF_RGBA16: return ERawImageFormat::RGBA16;
	case TSF_RGBA16F: return ERawImageFormat::RGBA16F;

	case TSF_G16: return ERawImageFormat::G16;
	case TSF_RGBA32F: return ERawImageFormat::RGBA32F;
	case TSF_R16F: return ERawImageFormat::R16F;
	case TSF_R32F: return ERawImageFormat::R32F;

	// these are mapped to TSF_BGRA8/TSF_BGRE8 on load, so the runtime will never see them :
	case TSF_RGBA8_DEPRECATED:
	case TSF_RGBE8_DEPRECATED:
		UE_LOGF(LogImageCoreUtils, Warning,"Deprecated format in ConvertToRawImageFormat not supported.");
		return ERawImageFormat::Invalid;

	default:
	case TSF_Invalid:
		check(0);
		return ERawImageFormat::Invalid;
	}
}

IMAGECORE_API ETextureSourceFormat FImageCoreUtils::ConvertToTextureSourceFormat(ERawImageFormat::Type Format)
{
	switch(Format)
	{
	case ERawImageFormat::G8:    return TSF_G8;
	case ERawImageFormat::BGRA8: return TSF_BGRA8;
	case ERawImageFormat::BGRE8: return TSF_BGRE8;
	case ERawImageFormat::RGBA16:  return TSF_RGBA16;
	case ERawImageFormat::RGBA16F: return TSF_RGBA16F;
	case ERawImageFormat::RGBA32F: return TSF_RGBA32F;
	case ERawImageFormat::G16: return TSF_G16;
	case ERawImageFormat::R16F:	return TSF_R16F;
	case ERawImageFormat::R32F:	return TSF_R32F;
	
	default:
		check(0);
		return TSF_Invalid;
	}
}


IMAGECORE_API FName FImageCoreUtils::ConvertToUncompressedTextureFormatName(ERawImageFormat::Type Format)
{
// from TextureFormatUncompressed.cpp :
#define ENUM_SUPPORTED_FORMATS(op) \
	op(BGRA8) \
	op(G8) \
	op(G16) \
	op(RGBA16F) \
	op(RGBA32F) \
	op(R16F) \
	op(R32F)
#define DECL_FORMAT_NAME(FormatName) static FName TextureFormatName##FormatName = FName(TEXT(#FormatName));
ENUM_SUPPORTED_FORMATS(DECL_FORMAT_NAME);
#undef DECL_FORMAT_NAME

	switch(Format)
	{
	case ERawImageFormat::G8:    return TextureFormatNameG8;
	case ERawImageFormat::BGRA8: return TextureFormatNameBGRA8;
	case ERawImageFormat::BGRE8: return TextureFormatNameRGBA16F; // <- not same
	case ERawImageFormat::RGBA16:  return TextureFormatNameBGRA8; // <- not same
	case ERawImageFormat::RGBA16F: return TextureFormatNameRGBA16F;
	case ERawImageFormat::RGBA32F: return TextureFormatNameRGBA32F;
	case ERawImageFormat::G16: return TextureFormatNameG16;
	case ERawImageFormat::R16F:	return TextureFormatNameR16F;
	case ERawImageFormat::R32F:	return TextureFormatNameR32F;
	
	default:
		check(0);
		return FName();
	}
}

IMAGECORE_API ETextureSourceFormat FImageCoreUtils::GetCommonSourceFormat(ETextureSourceFormat Format1, ETextureSourceFormat Format2)
{
	if (Format1 == Format2)
	{
		return Format1;
	}

	// calculation is based on the following oriented graph, where formats A and B are linked if conversion from A to B is considered to be acceptable, specifically:
	// TSF_G8 -> TSF_BGRA8 -> TSF_RGBA16
	// TSF_G8 -> TSF_G16 -> TSF_RGBA16
	// TSF_RGBA16 -> TSF_RGBA32F
	// TSF_R16F -> TSF_RGBA16F -> TSF_RGBA32F
	// TSF_R16F -> TSF_R32F -> TSF_RGBA32F
	// TSF_BGRE8 -> TSF_RGBA32F
	// the function returns the lowest possible parent node for two input nodes

	switch (Format1)
	{
	case TSF_G8:
		return !UE::TextureDefines::IsHDR(Format2) ? Format2 : TSF_RGBA32F;
	case TSF_G16:
	case TSF_BGRA8:
		return Format2 == TSF_G8 ? Format1 : !UE::TextureDefines::IsHDR(Format2) ? TSF_RGBA16 : TSF_RGBA32F;
	case TSF_RGBA16:
		return !UE::TextureDefines::IsHDR(Format2) ? Format1 : TSF_RGBA32F;
	case TSF_R16F:
		return Format2 == TSF_R32F || Format2 == TSF_RGBA16F ? Format2 : TSF_RGBA32F;
	case TSF_R32F:
	case TSF_RGBA16F:
		return Format2 == TSF_R16F ? Format1 : TSF_RGBA32F;
	default:
		return TSF_RGBA32F;
	}
}

IMAGECORE_API ERawImageFormat::Type FImageCoreUtils::GetRawImageFormatForPixelFormat(EPixelFormat PF)
{
	switch(PF)
	{
	case PF_Unknown:
	case PF_DepthStencil:
	case PF_ShadowDepth:
	case PF_D24                  :
		return ERawImageFormat::Invalid;

	case PF_A32B32G32R32F:
	case PF_G32R32F              :
	case PF_R32G32B32A32_UINT	:
	case PF_R32G32B32_UINT		:
	case PF_R32G32B32_SINT		:
	case PF_R32G32B32F			:
	case PF_R32G32_UINT         :
		return ERawImageFormat::RGBA32F;

	case PF_B8G8R8A8:
	case PF_R5G6B5_UNORM         :
	case PF_R8G8B8A8             :
	case PF_A8R8G8B8			:
	case PF_R8G8                :
	case PF_XGXR8				:
	case PF_R8G8B8A8_UINT		:
	case PF_R8G8B8A8_SNORM		:
	case PF_B5G5R5A1_UNORM       :
	case PF_G16R16_SNORM		:
	case PF_R8G8_UINT			:
		return ERawImageFormat::BGRA8;
		
	case PF_G8:
	case PF_L8	:
	case PF_R8_UINT				:
	case PF_R8		            :
		return ERawImageFormat::G8;

	case PF_G16:
		return ERawImageFormat::G16;

	case PF_FloatRGB:
	case PF_FloatRGBA:
	case PF_G16R16F              :
	case PF_G16R16F_FILTER       :
	case PF_FloatR11G11B10       :
	case PF_R9G9B9EXP5			:
		return ERawImageFormat::RGBA16F;
		
	case PF_R16F                 :
	case PF_R16F_FILTER          :
		return ERawImageFormat::R16F;

	case PF_R32_FLOAT:
	case PF_R32_UINT             :
	case PF_R32_SINT             :
	case PF_R8_SINT				:
	case PF_R64_UINT			:
		return ERawImageFormat::R32F;

	case PF_G16R16:
	case PF_A2B10G10R10          :
	case PF_A16B16G16R16         :
	case PF_R16_UINT             :
	case PF_R16_SINT             :
	case PF_R16G16B16A16_UINT    :
	case PF_R16G16B16A16_SINT    :
	case PF_R16G16_UINT			:
	case PF_R16G16B16A16_UNORM	:
	case PF_R16G16B16A16_SNORM	:
		return ERawImageFormat::RGBA16;
		
	case PF_DXT1:
	case PF_DXT3:
	case PF_DXT5:
	case PF_UYVY:
	case PF_BC5                  :
	case PF_V8U8                 :
	case PF_A1                   :
	case PF_A8                   :
	case PF_PVRTC2               :
	case PF_PVRTC4               :
	case PF_BC4					:
	case PF_ATC_RGB				:
	case PF_ATC_RGBA_E			:
	case PF_ATC_RGBA_I			:
	case PF_X24_G8				:
	case PF_ETC1				:
	case PF_ETC2_RGB			:
	case PF_ETC2_RGBA			:
	case PF_ASTC_4x4             :
	case PF_ASTC_6x6             :
	case PF_ASTC_8x8             :
	case PF_ASTC_10x10           :
	case PF_ASTC_12x12           :
	case PF_BC6H				:
	case PF_BC7					:
	case PF_PLATFORM_HDR_0		:
	case PF_PLATFORM_HDR_1		:
	case PF_PLATFORM_HDR_2		:
	case PF_NV12				:
	case PF_ETC2_R11_EAC		:
	case PF_ETC2_RG11_EAC		:
	case PF_ASTC_4x4_HDR         :
	case PF_ASTC_6x6_HDR         :
	case PF_ASTC_8x8_HDR         :
	case PF_ASTC_10x10_HDR       :
	case PF_ASTC_12x12_HDR       :
	case PF_P010				:
	case PF_ASTC_4x4_NORM_RG	:
	case PF_ASTC_6x6_NORM_RG	:
	case PF_ASTC_8x8_NORM_RG	:
	case PF_ASTC_10x10_NORM_RG	:
	case PF_ASTC_12x12_NORM_RG	:
		return ERawImageFormat::Invalid;

	default:
		check(0); // unexpected value
		return ERawImageFormat::Invalid;
	}
};
