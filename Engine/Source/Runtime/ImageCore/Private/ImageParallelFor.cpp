// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageParallelFor.h"

/**

FImage is tight packed in memory with slices adjacent to each other
so we can just treat it was a 2d image with Height *= NumSlices

@todo Oodle : actually because of the tight packed property, there's no need to use the original image dimensions at all
we could just cut into 1d runs of the desired pixel count.
That would give better parallelism on skew images (than keeping original SizeX which we do now).

eg. make "ImagePart" of 16384 pixels, and make "Rows" for the FLinearColor pass that are always exactly 512 pixels

*/

IMAGECORE_API int32 FImageCore::ImageParallelForComputeNumJobs(const FImageView & Image,int64 * pRowsPerJob)
{
	int64 SizeX = Image.SizeX;
	int64 SizeY = ImageParallelForComputeNumRows(Image);

	int32 RowsPerJob;
	int32 NumJobs = ImageParallelForComputeNumJobsForRows(RowsPerJob,SizeX,SizeY);

	check( (int64)NumJobs * RowsPerJob >= SizeY );
	check( (int64)(NumJobs-1) * RowsPerJob < SizeY );

	*pRowsPerJob = RowsPerJob;
	return NumJobs;
}

IMAGECORE_API int64 FImageCore::ImageParallelForMakePart(FImageView * Part,const FImageView & Image,int64 JobIndex,int64 RowsPerJob)
{
	int64 SizeX = Image.SizeX;
	int64 SizeY = ImageParallelForComputeNumRows(Image);

	int64 StartY = JobIndex * RowsPerJob;
	check( StartY < SizeY );

	int64 EndY = FMath::Min(StartY + RowsPerJob,SizeY);

	*Part = Image;
	Part->SizeY = EndY - StartY;
	Part->NumSlices = 1;
	Part->RawData = (uint8 *)Image.RawData + Image.GetBytesPerPixel() * SizeX * StartY;

	return StartY;
}

namespace
{

struct FLinearColorCmp
{
	inline bool operator () (const FLinearColor & lhs,const FLinearColor & rhs) const
	{
		if ( lhs.R != rhs.R ) return lhs.R < rhs.R;
		if ( lhs.G != rhs.G ) return lhs.G < rhs.G;
		if ( lhs.B != rhs.B ) return lhs.B < rhs.B;
		if ( lhs.A != rhs.A ) return lhs.A < rhs.A;
		return false;
	}
};

static inline FLinearColor SumColors(const TArrayView64<FLinearColor> & Colors)
{
	VectorRegister4Float VecSum = VectorSetFloat1(0.f);
	for ( FLinearColor & Color : Colors )
	{
		VecSum = VectorAdd(VecSum, VectorLoad(&Color.Component(0)));
	}

	FLinearColor Sum;
	VectorStore(VecSum,&Sum.Component(0));
	
	return Sum;
}

};

IMAGECORE_API FLinearColor FImageCore::ComputeImageLinearAverage(const FImageView & Image)
{
	TArray<FLinearColor> Accumulator_Rows;
	int64 Accumulator_RowCount = ImageParallelForComputeNumRows(Image);
	Accumulator_Rows.SetNum(Accumulator_RowCount);
	
	// just summing parallel portions to an accumulator would produce different output depending on thread count and timing
	//	because the float sums to accumulator are not order and grouping invariant
	// instead we are careful to ensure machine invariance
	// the image is cut into rows
	// each row is summed
	// then all those row sums are accumulated

	ImageParallelProcessLinearPixels(TEXT("PF.ComputeImageLinearAverage"),Image,
		[&](TArrayView64<FLinearColor> & Colors,int64 Y) 
		{
			// this is called once per row
			//	so it is always the same grouping of colors regardless of thread count
			
			FLinearColor Sum = SumColors(Colors);

			// do not just += on accumulator here because that would be an order-dependent race that changes output
			// instead we store all the row sums to later accumulate in known order

			Accumulator_Rows[Y] = Sum;

			return ProcessLinearPixelsAction::ReadOnly;
		}
	);

	FLinearColor Accumulator = SumColors(Accumulator_Rows);

	int64 NumPixels = Image.GetNumPixels();
	Accumulator *= (1.f / NumPixels);

	return Accumulator;
}

namespace
{
	// FMinMax helper for RGBA32F (FLinearColor) 4-float vector minmax

	struct FMinMax
	{
		VectorRegister4Float VMin,VMax;

		FMinMax() { }

		FMinMax(VectorRegister4Float InMin,VectorRegister4Float InMax) : VMin(InMin), VMax(InMax)
		{
		}
	};
	
	static inline FMinMax MinMax(const FMinMax & A,const FMinMax & B)
	{
		return FMinMax(
			VectorMin(A.VMin,B.VMin),
			VectorMax(A.VMax,B.VMax) );
	}

	static inline FMinMax MinMaxColors(const TArrayView64<FLinearColor> & Colors)
	{
		check( Colors.Num() > 0 );

		VectorRegister4Float VMin = VectorLoad(&Colors[0].Component(0));
		VectorRegister4Float VMax = VMin;
	
		for ( const FLinearColor & Color : Colors )
		{
			VectorRegister4Float VCur = VectorLoad(&Color.Component(0));

			VMin = VectorMin(VMin,VCur);
			VMax = VectorMax(VMax,VCur);
		}

		return FMinMax(VMin,VMax);
	}

};


static void ComputeChannelLinearMinMax_Generic(const FImageView & Image, FLinearColor & OutMin, FLinearColor & OutMax)
{
	check( Image.GetNumPixels() > 0 );

	// Generic routine works by converting the image to RGBA32F (FLinearColor)
	//	and using 4F vector minmax on that.
 
	if ( Image.GetNumPixels() <= 32 )
	{
		// fast path for tiny images (avoid allocs, also for the temp flinearcolor array, use stack)
		//	we want the fast path for up to 32 pixels/bytes

		TArrayView64<FLinearColor> Colors;		
		FLinearColor StackColors[32];

		if ( Image.Format == ERawImageFormat::RGBA32F )
		{
			Colors = Image.AsRGBA32F();
		}
		else
		{
			FImageView LinearView;
			LinearView = Image; // copy over image dimensions
			LinearView.Format = ERawImageFormat::RGBA32F;
			LinearView.GammaSpace = EGammaSpace::Linear;
			LinearView.RawData = (void *) &StackColors[0];

			FImageCore::CopyImage(Image,LinearView);

			Colors = LinearView.AsRGBA32F();
		}

		FMinMax MM = MinMaxColors(Colors);
		
		VectorStore(MM.VMin,&OutMin.Component(0));
		VectorStore(MM.VMax,&OutMax.Component(0));
	}
	else
	{
		TArray<FMinMax> MinMax_Rows;
		int64 MinMax_RowCount = FImageCore::ImageParallelForComputeNumRows(Image);
		MinMax_Rows.SetNum(MinMax_RowCount);
	
		FImageCore::ImageParallelProcessLinearPixels(TEXT("PF.ComputeChannelLinearMinMax"),Image,
			[&](TArrayView64<FLinearColor> & Colors,int64 Y) 
			{
				FMinMax Sum = MinMaxColors(Colors);

				MinMax_Rows[Y] = Sum;

				return FImageCore::ProcessLinearPixelsAction::ReadOnly;
			}
		);
	
		// now MinMax on all the rows :
		FMinMax NetMinMax = MinMax_Rows[0];
		for (const FMinMax & MM_Row : TArrayView64<FMinMax>(MinMax_Rows) ) // TArray iterator is slow, must wrap in ArrayView
		{
			NetMinMax = MinMax(NetMinMax, MM_Row );
		}

		VectorStore(NetMinMax.VMin,&OutMin.Component(0));
		VectorStore(NetMinMax.VMax,&OutMax.Component(0));
	}
}

namespace
{

union SixteenBytes
{
	uint8 Bytes[16]; 
	uint16 Words[8];
};

// fallback sixteen byte min-max helpers :

static inline SixteenBytes U8Min(const SixteenBytes & A,const SixteenBytes & B)
{
	SixteenBytes X;
	for(int64 i=0;i<16;i++)
	{
		X.Bytes[i] = FMath::Min<uint8>(A.Bytes[i],B.Bytes[i]);
	}
	return X;
}
static inline SixteenBytes U8Max(const SixteenBytes & A,const SixteenBytes & B)
{
	SixteenBytes X;
	for(int64 i=0;i<16;i++)
	{
		X.Bytes[i] = FMath::Max<uint8>(A.Bytes[i],B.Bytes[i]);
	}
	return X;
}

static inline SixteenBytes U16Min(const SixteenBytes & A,const SixteenBytes & B)
{
	SixteenBytes X;
	for(int64 i=0;i<8;i++)
	{
		X.Words[i] = FMath::Min<uint16>(A.Words[i],B.Words[i]);
	}
	return X;
}
static inline SixteenBytes U16Max(const SixteenBytes & A,const SixteenBytes & B)
{
	SixteenBytes X;
	for(int64 i=0;i<8;i++)
	{
		X.Words[i] = FMath::Max<uint16>(A.Words[i],B.Words[i]);
	}
	return X;
}

static void ComputeChannelLinearMinMax_Part_U8(const uint8 * PartStart,int64 Num16,SixteenBytes * OutMin,SixteenBytes * OutMax)
{
	check( Num16 > 0 );

	const SixteenBytes * P16 = (const SixteenBytes *)PartStart;
	
#if PLATFORM_CPU_X86_FAMILY

	__m128i VMin = _mm_loadu_si128((const __m128i*)P16);
	__m128i VMax = VMin;
	
	for(int64 i=1;i<Num16;i++)
	{
		__m128i Cur = _mm_loadu_si128((const __m128i*)&P16[i]);
		VMin = _mm_min_epu8(VMin,Cur);
		VMax = _mm_max_epu8(VMax,Cur);
	}

	_mm_storeu_si128((__m128i *)OutMin,VMin);
	_mm_storeu_si128((__m128i *)OutMax,VMax);

#else // fallback ; todo: ARM NEON ?

	SixteenBytes VMin,VMax;
	VMin = VMax = *P16;

	for(int64 i=1;i<Num16;i++)
	{
		VMin = U8Min(VMin,P16[i]);
		VMax = U8Max(VMax,P16[i]);
	}

	*OutMin = VMin;
	*OutMax = VMax;

#endif // PLATFORM SIMD
}

static void ComputeChannelLinearMinMax_Part_U16(const uint8 * PartStart,int64 Num16,SixteenBytes * OutMin,SixteenBytes * OutMax)
{
	check( Num16 > 0 );

	const SixteenBytes * P16 = (const SixteenBytes *)PartStart;
	
#if PLATFORM_CPU_X86_FAMILY

	__m128i VMin = _mm_loadu_si128((const __m128i*)P16);
	__m128i VMax = VMin;
	
	for(int64 i=1;i<Num16;i++)
	{
		__m128i Cur = _mm_loadu_si128((const __m128i*)&P16[i]);
		VMin = _mm_min_epu16(VMin,Cur); // SSE4.1
		VMax = _mm_max_epu16(VMax,Cur);
	}

	_mm_storeu_si128((__m128i *)OutMin,VMin);
	_mm_storeu_si128((__m128i *)OutMax,VMax);

#else // fallback ; todo: ARM NEON ?

	SixteenBytes VMin,VMax;
	VMin = VMax = *P16;

	for(int64 i=1;i<Num16;i++)
	{
		VMin = U16Min(VMin,P16[i]);
		VMax = U16Max(VMax,P16[i]);
	}

	*OutMin = VMin;
	*OutMax = VMax;

#endif // PLATFORM SIMD
}

} // namespace

void FImageCore::ComputeChannelLinearMinMax(const FImageView & Image, FLinearColor & OutMin, FLinearColor & OutMax)
{
	int64 NumPixels = Image.GetNumPixels();

	if ( NumPixels == 0 )
	{
		OutMin = OutMax = FLinearColor(ForceInit);
		return;
	}

	// fast path only works on U8 and U16 channels for now
	//	other pixel formats will use Generic fallback
	int64 PixelBytesPerChannel = 0;

	switch(Image.Format)
	{
	case ERawImageFormat::G8:
	case ERawImageFormat::BGRA8:
		PixelBytesPerChannel = 1;
		break;
	case ERawImageFormat::G16:
	case ERawImageFormat::RGBA16:
		PixelBytesPerChannel = 2;
		break;
	case ERawImageFormat::BGRE8:
	case ERawImageFormat::RGBA16F:
	case ERawImageFormat::RGBA32F:
	case ERawImageFormat::R16F:
	case ERawImageFormat::R32F:
		break;
	default:
		checkf(false,TEXT("ComputeChannelLinearMinMax Invalid pixel format"));
		//UE_LOGF(LogImageCore,Error,"ComputeChannelLinearMinMax Invalid pixel format");
		OutMin = OutMax = FLinearColor(ForceInit);
		return;
	}

	if ( PixelBytesPerChannel == 0 || NumPixels < 16 )
	{
		// unsupported format, or tiny, use generic
		ComputeChannelLinearMinMax_Generic(Image,OutMin,OutMax);
		return;
	}

	/**

	Design:

	FImage pixels are dense, so we just treat them as a bunch of samples of U8 or U16.
	(no need to look at width/height/slices at all).

	We ignore the pixel format other than knowing it is a U8 or U16 channel.
	We always work on 16-byte pieces, which can be varying number of pixels.

	Cut the data into pieces for parallel processing (16-byte aligned).
	Find the 16-byte vector min/max on those pieces.
	Then gather min/max of the 16-bytes from each piece.

	For the tail portion that may not be a full 16 bytes, we can just replicate a pixel to fill 16 bytes
	and use the same 16-byte aligned min/max routines, so no special tail case is required.

	Finally once we have the 16-byte min/max of the whole image, we run the Generic fallback
	which does the horizontal min/max inside that vector, and also handles correctly interpreting whether
	it is 16xG8 or 4xBGRA8 or whatever.

	**/

	int64 BytesPerPixel = Image.GetBytesPerPixel();
	int64 PixelsPer16 = 16 / (uint32)BytesPerPixel;

	// integer number of pixels per 16 bytes :
	check( (PixelsPer16 * BytesPerPixel) == 16 );

	int64 ImageSizeBytes = Image.GetImageSizeBytes();
	check( (BytesPerPixel * NumPixels) == ImageSizeBytes );

	// divide image into chunks for parallel processing
	// each chunk is a multiple of 16 bytes
	// final chunk may be less than 16 bytes

	int64 ImageSizeBytes16 = ImageSizeBytes & (~(int64)0xF);
	bool Non16Tail = (ImageSizeBytes != ImageSizeBytes16);

	int64 NumPixelsPerJob = 0;
	int32 NumJobs = ImageParallelForComputeNumJobsForPixels(NumPixelsPerJob,NumPixels);
	int64 BytesPerJob = NumPixelsPerJob * BytesPerPixel;

	// round up to next 16 :
	BytesPerJob = (BytesPerJob + 0xF) & (~(int64)0xF);

	// Number of 16-byte min/maxes is NumJobs plus 1 more if a non-16-byte-aligned tail is present
	int64 NumSixteens = NumJobs + ( Non16Tail ? 1 : 0 );
	TArray64<SixteenBytes> MinMaxes;
	MinMaxes.SetNumUninitialized( NumSixteens*2 );
	SixteenBytes * VMins = &MinMaxes[0];
	SixteenBytes * VMaxs = &MinMaxes[NumSixteens];

	uint8 * ImageBytes = (uint8 *) Image.RawData;

	ParallelFor(TEXT("PF.ComputeChannelLinearMinMax"), NumJobs, 1, [&](int64 JobIndex)
	{
		int64 JobStartBytes = JobIndex * BytesPerJob;
		int64 JobNumBytes = FMath::Min( BytesPerJob, (ImageSizeBytes16 - JobStartBytes) );

		check( JobNumBytes > 0 );
		check( (JobStartBytes & 0xF) == 0 );
		check( (JobNumBytes & 0xF) == 0 );

		const uint8 * PartStart = ImageBytes + JobStartBytes;
		int64 JobNum16 = JobNumBytes>>4;

		check( PixelBytesPerChannel == 1 || PixelBytesPerChannel == 2 );

		if ( PixelBytesPerChannel == 1 )
		{
			ComputeChannelLinearMinMax_Part_U8(PartStart,JobNum16,VMins+JobIndex,VMaxs+JobIndex);
		}
		else
		{
			ComputeChannelLinearMinMax_Part_U16(PartStart,JobNum16,VMins+JobIndex,VMaxs+JobIndex);
		}

	}, EParallelForFlags::Unbalanced);

	if ( Non16Tail )
	{
		// there's a non-16 byte aligned tail
		// replicate a pixel to fill a 16-byte VMin/VMax in the [NumJobs] slot
		check( NumSixteens == NumJobs + 1 );

		uint8 * Tail16 = ImageBytes + ImageSizeBytes16;
		int64 TailSize = ImageSizeBytes - ImageSizeBytes16;
		check( TailSize > 0 && TailSize < 16 );

		SixteenBytes Tail;
		memcpy(Tail.Bytes,Tail16,TailSize);

		uint8 * TailPtr = Tail.Bytes + TailSize;
		uint8 * TailEnd = Tail.Bytes + 16;
		while( TailPtr < TailEnd )
		{
			memcpy(TailPtr,Tail.Bytes,BytesPerPixel);
			TailPtr += BytesPerPixel;
		}
		check( TailPtr == TailEnd );

		VMins[NumJobs] = Tail;
		VMaxs[NumJobs] = Tail;
	}

	// now do minmax over the VMins/VMaxs
	//  we want the Min of VMins and the Max of VMaxs in Accum[]
	//	 we also compute the Max/Min which are not needed and just discard in Junk
	//	 this is a tiny bit of extra work (we could just do min on mins) but whatever
	SixteenBytes Accum[2];
	
	if ( PixelBytesPerChannel == 1 )
	{
		ComputeChannelLinearMinMax_Part_U8(MinMaxes[0].Bytes,NumSixteens*2,&Accum[0],&Accum[1]);
	}
	else
	{
		ComputeChannelLinearMinMax_Part_U16(MinMaxes[0].Bytes,NumSixteens*2,&Accum[0],&Accum[1]);
	}

	// Accum now has the minmaxes
	// it is always 32 bytes
	// reinterpret it as an ImageView to do the pixel format conversion
	//	 (this also does the min/max within the 16-byte vectors)
	// start by copying Image to get formats
	FImageView AccumView(Image);
	AccumView.RawData = Accum[0].Bytes;
	AccumView.NumSlices = 1;
	AccumView.SizeY = 1;
	AccumView.SizeX = 32 / (uint32)BytesPerPixel;
	check( AccumView.GetImageSizeBytes() == 32 );

	ComputeChannelLinearMinMax_Generic(AccumView,OutMin,OutMax);
}

bool FImageCore::ScaleChannelsSoMinMaxIsInZeroToOne(const FImageView & Image)
{
	if ( Image.GetNumPixels() == 0 )
	{
		return false;
	}
	if ( ! ERawImageFormat::IsHDR(Image.Format) )
	{
		// early out : if Image is U8/U16 it is already in [0,1]
		return false;
	}

	FLinearColor Min,Max;
	ComputeChannelLinearMinMax(Image,Min,Max);

	if ( Min.R >= 0.f && Min.G >= 0.f && Min.B >= 0.f && Min.A >= 0.f &&
		Max.R <= 1.f && Max.G <= 1.f && Max.B <= 1.f && Max.A <= 1.f )
	{
		// nothing to do
		return false;
	}

	VectorRegister4Float VMin = VectorLoad(&Min.Component(0));
	VectorRegister4Float VMax = VectorLoad(&Max.Component(0));

	// this makes it so that the end of the range that was already in [0,1] is not modified :
	VMin = VectorMin( VMin, MakeVectorRegisterFloat(0.f,0.f,0.f,0.f) );
	VMax = VectorMax( VMax, MakeVectorRegisterFloat(1.f,1.f,1.f,1.f) );

	// VScale = 1.f/(Max-Min)
	VectorRegister4Float VSub = VectorSubtract(VMax,VMin);
	// avoid divide by zero :
	VSub = VectorMax(VSub, MakeVectorRegisterFloat(FLT_MIN,FLT_MIN,FLT_MIN,FLT_MIN) );
	VectorRegister4Float VScale = VectorReciprocalAccurate(VSub);

	ImageParallelProcessLinearPixels(TEXT("PF.ScaleChannelsSoMinMaxIsInZeroToOne"),Image,
		[&](TArrayView64<FLinearColor> & Colors,int64 Y) 
		{
			for ( FLinearColor & Color : Colors )
			{
				VectorRegister4Float VCur = VectorLoad(&Color.Component(0));

				VCur = VectorSubtract(VCur,VMin);
				VCur = VectorMultiply(VCur,VScale);

				VectorStore(VCur, &Color.Component(0));
			}

			return ProcessLinearPixelsAction::Modified;
		}
	);

	return true;
}
