// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "CoreTypes.h"
#include "ImageCore.h"
#include "Math/Color.h"

// This is supposed to be, as close as possible, the complete list of DXGI formats
// per those docs - the ODDFMTs we don't support for doing anything other than name, but we keep
// them here so we can stay in sync with the docs more easily.
//
// we support nope out on some of the more exotic formats entirely where they're unlikely
// to be useful for any assets
//
// fields: name, ID, bytes per unit (unit=1 texel for RGB, 1 block for BCN)
#define UE_DXGI_FORMAT_LIST \
	RGBFMT(UNKNOWN,						0,	0) \
	RGBFMT(R32G32B32A32_TYPELESS,		1,	16) \
	RGBFMT(R32G32B32A32_FLOAT,			2,	16) \
	RGBFMT(R32G32B32A32_UINT,			3,	16) \
	RGBFMT(R32G32B32A32_SINT,			4,	16) \
	RGBFMT(R32G32B32_TYPELESS,			5,	12) \
	RGBFMT(R32G32B32_FLOAT,				6,	12) \
	RGBFMT(R32G32B32_UINT,				7,	12) \
	RGBFMT(R32G32B32_SINT,				8,	12) \
	RGBFMT(R16G16B16A16_TYPELESS,		9,	8) \
	RGBFMT(R16G16B16A16_FLOAT,			10, 8) \
	RGBFMT(R16G16B16A16_UNORM,			11, 8) \
	RGBFMT(R16G16B16A16_UINT,			12, 8) \
	RGBFMT(R16G16B16A16_SNORM,			13, 8) \
	RGBFMT(R16G16B16A16_SINT,			14, 8) \
	RGBFMT(R32G32_TYPELESS,				15, 8) \
	RGBFMT(R32G32_FLOAT,				16, 8) \
	RGBFMT(R32G32_UINT,					17, 8) \
	RGBFMT(R32G32_SINT,					18, 8) \
	RGBFMT(R32G8X24_TYPELESS,			19, 8) \
	RGBFMT(D32_FLOAT_S8X24_UINT,		20, 8) \
	RGBFMT(R32_FLOAT_X8X24_TYPELESS,	21, 8) \
	RGBFMT(X32_TYPELESS_G8X24_UINT,		22, 8) \
	RGBFMT(R10G10B10A2_TYPELESS,		23, 4) \
	RGBFMT(R10G10B10A2_UNORM,			24, 4) \
	RGBFMT(R10G10B10A2_UINT,			25, 4) \
	RGBFMT(R11G11B10_FLOAT,				26, 4) \
	RGBFMT(R8G8B8A8_TYPELESS,			27, 4) \
	RGBFMT(R8G8B8A8_UNORM,				28, 4) \
	RGBFMT(R8G8B8A8_UNORM_SRGB,			29, 4) \
	RGBFMT(R8G8B8A8_UINT,				30, 4) \
	RGBFMT(R8G8B8A8_SNORM,				31, 4) \
	RGBFMT(R8G8B8A8_SINT,				32, 4) \
	RGBFMT(R16G16_TYPELESS,				33, 4) \
	RGBFMT(R16G16_FLOAT,				34, 4) \
	RGBFMT(R16G16_UNORM,				35, 4) \
	RGBFMT(R16G16_UINT,					36, 4) \
	RGBFMT(R16G16_SNORM,				37, 4) \
	RGBFMT(R16G16_SINT,					38, 4) \
	RGBFMT(R32_TYPELESS,				39, 4) \
	RGBFMT(D32_FLOAT,					40, 4) \
	RGBFMT(R32_FLOAT,					41, 4) \
	RGBFMT(R32_UINT,					42, 4) \
	RGBFMT(R32_SINT,					43, 4) \
	RGBFMT(R24G8_TYPELESS,				44, 4) \
	RGBFMT(D24_UNORM_S8_UINT,			45, 4) \
	RGBFMT(R24_UNORM_X8_TYPELESS,		46, 4) \
	RGBFMT(X24_TYPELESS_G8_UINT,		47, 4) \
	RGBFMT(R8G8_TYPELESS,				48, 2) \
	RGBFMT(R8G8_UNORM,					49, 2) \
	RGBFMT(R8G8_UINT,					50, 2) \
	RGBFMT(R8G8_SNORM,					51, 2) \
	RGBFMT(R8G8_SINT,					52, 2) \
	RGBFMT(R16_TYPELESS,				53, 2) \
	RGBFMT(R16_FLOAT,					54, 2) \
	RGBFMT(D16_UNORM,					55, 2) \
	RGBFMT(R16_UNORM,					56, 2) \
	RGBFMT(R16_UINT,					57, 2) \
	RGBFMT(R16_SNORM,					58, 2) \
	RGBFMT(R16_SINT,					59, 2) \
	RGBFMT(R8_TYPELESS,					60, 1) \
	RGBFMT(R8_UNORM,					61, 1) \
	RGBFMT(R8_UINT,						62, 1) \
	RGBFMT(R8_SNORM,					63, 1) \
	RGBFMT(R8_SINT,						64, 1) \
	RGBFMT(A8_UNORM,					65, 1) \
	ODDFMT(R1_UNORM,					66) \
	RGBFMT(R9G9B9E5_SHAREDEXP,			67, 4) \
	ODDFMT(R8G8_B8G8_UNORM,				68) \
	ODDFMT(G8R8_G8B8_UNORM,				69) \
	BCNFMT(BC1_TYPELESS,				70, 8) \
	BCNFMT(BC1_UNORM,					71, 8) \
	BCNFMT(BC1_UNORM_SRGB,				72, 8) \
	BCNFMT(BC2_TYPELESS,				73, 16) \
	BCNFMT(BC2_UNORM,					74, 16) \
	BCNFMT(BC2_UNORM_SRGB,				75, 16) \
	BCNFMT(BC3_TYPELESS,				76, 16) \
	BCNFMT(BC3_UNORM,					77, 16) \
	BCNFMT(BC3_UNORM_SRGB,				78, 16) \
	BCNFMT(BC4_TYPELESS,				79, 8) \
	BCNFMT(BC4_UNORM,					80, 8) \
	BCNFMT(BC4_SNORM,					81, 8) \
	BCNFMT(BC5_TYPELESS,				82, 16) \
	BCNFMT(BC5_UNORM,					83, 16) \
	BCNFMT(BC5_SNORM,					84, 16) \
	RGBFMT(B5G6R5_UNORM,				85, 2) \
	RGBFMT(B5G5R5A1_UNORM,				86, 2) \
	RGBFMT(B8G8R8A8_UNORM,				87, 4) \
	RGBFMT(B8G8R8X8_UNORM,				88, 4) \
	RGBFMT(R10G10B10_XR_BIAS_A2_UNORM,	89, 4) \
	RGBFMT(B8G8R8A8_TYPELESS,			90, 4) \
	RGBFMT(B8G8R8A8_UNORM_SRGB,			91, 4) \
	RGBFMT(B8G8R8X8_TYPELESS,			92, 4) \
	RGBFMT(B8G8R8X8_UNORM_SRGB,			93, 4) \
	BCNFMT(BC6H_TYPELESS,				94, 16) \
	BCNFMT(BC6H_UF16,					95, 16) \
	BCNFMT(BC6H_SF16,					96, 16) \
	BCNFMT(BC7_TYPELESS,				97, 16) \
	BCNFMT(BC7_UNORM,					98, 16) \
	BCNFMT(BC7_UNORM_SRGB,				99, 16) \
	ODDFMT(AYUV,						100) \
	ODDFMT(Y410,						101) \
	ODDFMT(Y416,						102) \
	ODDFMT(NV12,						103) \
	ODDFMT(P010,						104) \
	ODDFMT(P016,						105) \
	ODDFMT(_420_OPAQUE,					106) \
	ODDFMT(YUY2,						107) \
	ODDFMT(Y210,						108) \
	ODDFMT(Y216,						109) \
	ODDFMT(NV11,						110) \
	ODDFMT(AI44,						111) \
	ODDFMT(IA44,						112) \
	ODDFMT(P8,							113) \
	ODDFMT(A8P8,						114) \
	RGBFMT(B4G4R4A4_UNORM,				115,2) \
	ODDFMT(P208,						130) \
	ODDFMT(V208,						131) \
	ODDFMT(V408,						132) \
	RGBFMT(B8G8R8,						1001,3) \
	RGBFMT(R8G8B8,						1002,3) \
	RGBFMT(R8G8B8X8,					1003,4) \
	/* end */

namespace UE { namespace DDS
{
	// Complete list of formats
	enum class EDXGIFormat 
	{
	#define RGBFMT(name,id,bypu) name = id,
	#define BCNFMT(name,id,bypu) name = id,
	#define ODDFMT(name,id) name = id,
		UE_DXGI_FORMAT_LIST
	#undef RGBFMT
	#undef BCNFMT
	#undef ODDFMT
	};

	// What version of DDS file to write
	enum class EDDSFormatVersion
	{
		Auto,	// Write as D3D9-version DDS if possible without losing metadata like sRGB status, D3D10 otherwise
		D3D9,	// Try really hard to write a D3D9-format .DDS file (the more widely supported version) even when doing so loses some information
		D3D10,	// Always write a D3D10-format .DDS file
	};

	// Mip data read modes
	enum class EDDSReadMipMode
	{
		Full,
		HeaderOnly,
		HeaderWithMipInfo,
	};

	// Error codes
	enum class EDDSError
	{
		OK,						// No error
		Ok = OK,
		OutOfMemory,			// Out of memory/allocation failure while reading
		NotADds,				// No valid DDS header found
		NotADDS = NotADds,
		BadResourceDimension,	// Resource not 1D, 2D or 3D
		BadPixelFormat,			// Malformed or unsupported pixel format
		BadImageDimension,		// Image dimensions (width, height, depth, array size) outside supported range
		BadMipmapCount,			// Invalid mipmap count for given image dimensions
		BadCubemap,				// Cubemap has non-square faces, non-1 depth, or unsupported array count
		IoError,				// General I/O error
	};

	// Channel ordering for 8-bit formats
	enum class EChannelOrder
	{
		RGBA,
		BGRA
	};

	// One of these for each mip of a DDS
	struct FDDSMip
	{
		uint32 Width, Height, Depth;
		// Note: pixels in DDS data must be densely packed, they exist for ease of addressing
		// but you can't represent strided data in this form.
		int64 RowStride;		// Bytes in one row of a 2d slice of the mip. equal to SliceStride for 1D textures.
		int64 SliceStride;		// Bytes in one 2d slice of the mip, equal to DataStride for 2D textures.
		int64 DataSize;
		uint8* Data;			// just raw bytes; interpretation as per DXGIFormat in parent FDDSFile
	};

	// Metadata structure for a DDS file, with access pointers for the raw texture data (i.e. 
	// unconverted from the DXGI format).
	//
	// Can be used for loading or saving a dds file.
	//
	// For saving, general structure is:
	//	{
	//		dds = FDDSFile::CreateEmpty[2D](...)
	//		copy data for each mip/array:
	//		for (mips)
	//			memcpy(dds->mips[idx].data, mip_data, mip_size);
	// 
	//		Ar = IFileManager::Get().CreateFileWriter(filename);
	// 
	//		dds->Serialize(Ar);
	//		delete dds;
	// 
	//		Ar->Close()
	//		delete Ar;
	//	}
	struct FDDSFile
	{
		int32 Dimension=0; // 1, 2 or 3.
		uint32 Width=0;
		uint32 Height=0;
		uint32 Depth=0;
		uint32 MipCount=0;

		// Array size here uses the conventions as in the D3D runtime, which means
		// an array of N cubemaps has an array size of 6N (one element per face).
		uint32 ArraySize=0;
		EDXGIFormat DXGIFormat=EDXGIFormat::UNKNOWN;
		uint32 CreateFlags=0;

		// Mips are ordered starting from mip 0 (full-size texture) decreasing in size;
		// for arrays, we store the full mip chain for one array element before advancing
		// to the next array element, and have (ArraySize * MipCount) mips total.
		TArray<FDDSMip> Mips;

		// When we allocate the mips ourselves, we keep all storage together in a single
		// allocation, namely this one, and the mips point into it.
		// With CREATE_FLAG_NO_MIP_STORAGE_ALLOC, we don't own the underlying memory and
		// this array is unused.
		TArray64<uint8> MipRawData;

		// Write DDS-format data to memory. This produces a memory image of a DDS file.
		//
		// Automatic format selection rules used are:
		// - D3D9 DDS does not support 1D or array textures directly, so these always write as D3D10
		// - The pixel format must be a "common" D3D9 format (more obscure formats can be used for D3D9
		//   writing when InFormatVersion is D3D9, but "Auto" sticks with a restricted subset)
		// - For 8-bit pixel formats, the format must be non-sRGB.
		// 
		// The latter is somewhat arbitrary. D3D9 DDS files don't store whether pixels for a texture are meant
		// to be interpreted as sRGB or not. This reader treats the resulting formats as non-sRGB UNORM,
		// so the writer does the same for symmetry.
		IMAGECORE_API EDDSError WriteDDS(TArray64<uint8>& OutDDS, EDDSFormatVersion InFormatVersion=EDDSFormatVersion::D3D10);

		// 20 mips means 512k x 512k pixels max, should be sufficient for now.
		// It's 512k not 1M because the final 1x1 pixel mip counts; i.e. a mip chain
		// from 1048576x1048576 down to 1x1 has 21 mips!
		static constexpr uint32 MAX_MIPS_SUPPORTED = 20;

		// Create an empty DDS structure (for writing DDS files typically)
		//
		// InDimension is [1,3] (1D,2D,3D)
		// InWidth/InHeight are for the top mip. Cubemaps must be square.
		// InDepth is only used for 3D textures, must be 1 otherwise.
		// InMipCount <= MAX_MIPS_SUPPORTED.
		// InArraySize number of textures provided - must be multiple of 6 for cubemaps (CREATE_FLAG_CUBEMAP)
		// 
		// Note for texture arrays the mip data pointers are expecting the entire mip chain 
		// for one texture before moving to the next
		//
		// On error, returns nullptr. If a non-null OutError is supplied, error information
		// is written there.
		IMAGECORE_API static FDDSFile* CreateEmpty(int32 InDimension, uint32 InWidth, uint32 InHeight, uint32 InDepth, uint32 InMipCount, uint32 InArraySize, EDXGIFormat InFormat, uint32 InCreateFlags, EDDSError *OutError=nullptr);

		// Convenience version of the above to create a basic 2D texture with mip chain
		IMAGECORE_API static FDDSFile* CreateEmpty2D(uint32 InWidth, uint32 InHeight, uint32 InMipCount, EDXGIFormat InFormat, uint32 InCreateFlags, EDDSError* OutError=nullptr);

		// Sanity-check that all members make sense and return an error code.
		IMAGECORE_API EDDSError Validate() const;

		// Check if this DDS is usable in Unreal Texture types:
		IMAGECORE_API bool IsValidTexture2D() const;
		IMAGECORE_API bool IsValidTextureCube() const; // or cube array
		IMAGECORE_API bool IsValidTextureArray() const;
		IMAGECORE_API bool IsValidTextureVolume() const;

		UE_DEPRECATED(5.6, "This function is now deprecated, please use CreateFromDDSInMemory with EDDSReadMipMode instead.")
		IMAGECORE_API static FDDSFile* CreateFromDDSInMemory(const uint8* InDDS, int64 InDDSSize, EDDSError* OutError, bool bHeaderOnly);

		// Used for loading from a DDS file image in memory.
		// 
		// On error, returns nullptr. If a non-null OutError is supplied, error information
		// is written there.
		IMAGECORE_API static FDDSFile* CreateFromDDSInMemory(const uint8* InDDS, int64 InDDSSize, EDDSError* OutError=nullptr, EDDSReadMipMode InMipReadMode = EDDSReadMipMode::Full);
		
		// is this buffer a DDS ?
		IMAGECORE_API static bool IsADDS(const uint8* InDDS, int64 InDDSSize);

		// For 8-bit UNorm formats, both RGBA and BGRA channel order variants exist and
		// are popular. This function does the trivial conversion between the two, updating
		// both the DXGI Format and the pixel data to do so. When the channel order is already
		// correct or the pixel format is not one of the few formats that exist in both RGBA
		// and BGRA variants, this function does nothing.
		IMAGECORE_API void ConvertChannelOrder(EChannelOrder InTargetOrder);

		// convert X alpha to A  and fill 255
		IMAGECORE_API void ConvertRGBXtoRGBA();		

		// Blit pixels from DDS mip to Image
		// return false if no pixel format conversion is supported
		// ToImage is not allocated, must be pre-allocated and sized correctly
		IMAGECORE_API bool GetMipImage(const FImageView & ToImage, int MipIndex) const;

		// Fill DDS mip from Image
		// Image must be of format DXGIFormatFromRawFormat
		IMAGECORE_API void FillMip(const FImageView & FromImage, int MipIndex);

		// Bit flags
		static constexpr uint32 CREATE_FLAG_NONE = 0;
		static constexpr uint32 CREATE_FLAG_CUBEMAP = 1;
		static constexpr uint32 CREATE_FLAG_NO_MIP_STORAGE_ALLOC = 2; // Mip storage wasn't allocated by us and isn't owned by us, just pointing elsewhere.
		static constexpr uint32 CREATE_FLAG_WAS_D3D9 = 4; // Loaded from D3D9-format .DDS file, so whether 8-bit/channel formats are sRGB or not is largely guesswork
	};

	// Returns the name of a DXGI format
	IMAGECORE_API const TCHAR *DXGIFormatGetName(EDXGIFormat InFormat);

	// Returns whether a given pixel format is sRGB
	IMAGECORE_API bool DXGIFormatIsSRGB(EDXGIFormat InFormat);
	IMAGECORE_API bool DXGIFormatHasLinearAndSRGBForm(EDXGIFormat InFormat);

	// Return the corresponding non-sRGB version of a pixel format if there is one
	IMAGECORE_API EDXGIFormat DXGIFormatRemoveSRGB(EDXGIFormat InFormat);

	// Return the corresponding sRGB version of a pixel format if there is one
	IMAGECORE_API EDXGIFormat DXGIFormatAddSRGB(EDXGIFormat InFormat);

	IMAGECORE_API EDXGIFormat DXGIFormatFromRawFormat(ERawImageFormat::Type RawFormat,EGammaSpace GammaSpace);

	IMAGECORE_API ERawImageFormat::Type DXGIFormatGetClosestRawFormat(EDXGIFormat fmt, bool * pIsExactMatch = nullptr);
	
	// note these sizes include the 4-byte magic word, unlike the older DDSLoader functions
	IMAGECORE_API int64 GetDDSHeaderMaximalSize();
	IMAGECORE_API int64 GetDDSHeaderMinimalSize();

} }
