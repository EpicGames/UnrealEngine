// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/****
* 
* ImageCore : low level pixel surface types
* 
* FImage owns an array of pixels
* FImageView can point at an array of pixels (FImage or otherwise)
* both have an FImageInfo
* 
* ERawImageFormat::Type is a pixel format that can be used in FImage (or TextureSource)
* 
* helpers to load/save/convert FImage are in Engine in FImageUtils
* 
* ImageCore does not use Engine, only Core
* it can be used in standalone apps that don't have Engine
*
* note to authors: as much as possible, write functions that act on FImageView
*  for example reading and modifying pixels? use FImageView
* use FImage when you may need to change the format or allocate a new image.
*
* prefer using FImage/FImageView instead of TextureSourceFormat or raw arrays or bytes
*
* Try not to write code that switches on pixel format, as formats may be added and it creates a fragile maintenance problem.
*
*/


/* Dependencies
 *****************************************************************************/

#include "Containers/ContainersFwd.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Math/Color.h"
#include "Math/Float16.h"
#include "Math/Float16Color.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"

/* Types
 *****************************************************************************/

 struct FImage;

// EGammaSpace is in Color.h

namespace ERawImageFormat
{
	/**
	 * Enumerates supported raw image formats.
	 * should map one-to-one with ETextureSourceFormat
	 */
	enum Type : uint8
	{
		G8,
		BGRA8,   // FColor
		BGRE8,
		RGBA16,
		RGBA16F, // FFloat16Color
		RGBA32F, // FLinearColor
		G16,     // note G8/G16 = gray = replicate to 3 channels, R16F = just in red channel
		R16F,
		R32F,
		MAX, // used for validation checks < MAX = valid type.
		Invalid = 0xFF
	};
	
	// RGBA -> G8 takes from the  *R* channel
	// G8 -> RGBA replicated to gray
 
	IMAGECORE_API int64 GetBytesPerPixel(Type Format);
	
	IMAGECORE_API const TCHAR * GetName(Type Format);
	IMAGECORE_API const FUtf8StringView GetNameView(Type Format);
	IMAGECORE_API bool GetFormatFromString(FUtf8StringView InString, Type& OutFormat);
	
	// IsU8Channels is not true for BGRE8, it means channels of uint8 in [0,255]
	IMAGECORE_API bool IsU8Channels(Type Format);
	IMAGECORE_API int NumChannels(Type Format);
	IMAGECORE_API bool IsHDR(Type Format);
	IMAGECORE_API bool HasAlphaChannel(Type Format);
	
	// Get one pixel of Format type from PixelData and return in Linear color
	IMAGECORE_API const FLinearColor GetOnePixelLinear(const void * PixelData,Type Format,EGammaSpace Gamma);
	
	// Get one pixel of Format type from PixelData and return in Linear color
	inline const FLinearColor GetOnePixelLinear(const void * PixelData,Type Format,bool bSRGB)
	{
		return GetOnePixelLinear(PixelData,Format,bSRGB ? EGammaSpace::sRGB : EGammaSpace::Linear);
	}

	// G8 and BGRA8 are affected by Gamma, 16/32 is NOT (they are always Linear)
	// this happens to be the same as IsU8Channels() at the moment, but don't assume that
	inline bool GetFormatNeedsGammaSpace(Type Format)
	{
		if ( Format == G8 || Format == BGRA8 )
		{
			return true;
		}
		else
		{
			// these formats ignore GammaSpace setting, always Linear
			return false;
		}
	}

	// when converting from pixel bags that don't store gamma to FImage
	//	you can use this to guess the gamma they probably wanted
	//	do not use when a real gamma flag is available!
	inline EGammaSpace GetDefaultGammaSpace(Type Format)
	{
		if ( GetFormatNeedsGammaSpace(Format) )
		{
			return EGammaSpace::sRGB;
		}
		else
		{
			return EGammaSpace::Linear;
		}
	}
};

namespace UE {
	namespace Color {
		enum class EChromaticAdaptationMethod : uint8;
	}
}

/**
* 
* FImageInfo describes a 2d pixel surface
* it can be used by FImage or FImageView
* 
*/
struct FImageInfo
{
	/** Width of the image. */
	int32 SizeX = 0;
	
	/** Height of the image. */
	int32 SizeY = 0;
	
	/** Number of image slices. */
	int32 NumSlices = 0;
	
	/** Format in which the image is stored. */
	ERawImageFormat::Type Format = ERawImageFormat::BGRA8;
	
	/** The gamma space the image is stored in. */
	EGammaSpace GammaSpace = EGammaSpace::sRGB;

	FImageInfo() { }
	
	FImageInfo(int32 InSizeX, int32 InSizeY, int32 InNumSlices, ERawImageFormat::Type InFormat, EGammaSpace InGammaSpace)
		: SizeX(InSizeX)
		, SizeY(InSizeY)
		, NumSlices(InNumSlices)
		, Format(InFormat)
		, GammaSpace(InGammaSpace)
	{
	}
	
	inline bool operator == (const FImageInfo & rhs) const
	{
		return 
			SizeX == rhs.SizeX &&
			SizeY == rhs.SizeY &&
			NumSlices == rhs.NumSlices &&
			Format == rhs.Format &&
			GammaSpace == rhs.GammaSpace;
	}

	inline bool IsImageInfoValid() const
	{
		if ( SizeX < 0 || SizeY < 0 || NumSlices < 0 ) return false;
		if ( Format == ERawImageFormat::Invalid ) return false;
		if ( GammaSpace == EGammaSpace::Invalid ) return false;
		if ( ! GetFormatNeedsGammaSpace(Format) && GammaSpace != EGammaSpace::Linear ) return false;
		return true;
	}

	inline bool IsGammaCorrected() const
	{
		return GammaSpace != EGammaSpace::Linear;
	}

	/**
	 * Gets the number of bytes per pixel.
	 *
	 * @return Bytes per pixel.
	 */
	inline int64 GetBytesPerPixel() const
	{
		return ERawImageFormat::GetBytesPerPixel(Format);
	}
	
	inline int64 GetNumPixels() const
	{
		return (int64) SizeX * SizeY * NumSlices;
	}
	
	inline int64 GetImageSizeBytes() const
	{
		return GetNumPixels() * GetBytesPerPixel();
	}
	
	inline int64 GetSliceNumPixels() const
	{
		return (int64) SizeX * SizeY;
	}
	
	inline int64 GetSliceSizeBytes() const
	{
		return GetSliceNumPixels() * GetBytesPerPixel();
	}
	
	inline int64 GetWidth()  const { return SizeX; }
	inline int64 GetHeight() const { return SizeY; }
	
	inline int64 GetStrideBytes()  const { return SizeX * GetBytesPerPixel(); }

	inline EGammaSpace GetGammaSpace() const
	{
		// Gamma is ignored unless GetFormatNeedsGammaSpace, so make sure it is Linear
		check( GetFormatNeedsGammaSpace(Format) || GammaSpace == EGammaSpace::Linear );

		return GammaSpace;
	}
	
	// get offset of a pixel from the base pointer, in bytes
	inline int64 GetPixelOffsetBytes(int32 X,int32 Y,int32 Slice = 0) const
	{
		checkSlow( X >= 0 && X < SizeX );
		checkSlow( Y >= 0 && Y < SizeY );
		checkSlow( Slice >= 0 && Slice < NumSlices );
		
		int64 Offset = Slice * GetSliceNumPixels();
		Offset += Y * (int64)SizeX;
		Offset += X;
		// Offset is now in pixels
		Offset *= GetBytesPerPixel();

		return Offset;
	}

	IMAGECORE_API void ImageInfoToCompactBinary(class FCbObject& OutObject) const;
	// Overwrites the current info with the object's info/
	IMAGECORE_API bool ImageInfoFromCompactBinary(const FCbObject& InObject);
};

/***
* 
* Image functions should take an FImageView as input and output to FImage
* 
* FImageView can point at an FImage or any pixel surface
* 
* FImage hold allocations.  FImageView is non-owning.  Make sure the memory pointed at is not freed.
* 
*/
struct FImageView : public FImageInfo
{
	void * RawData = nullptr;

	FImageView() { }
	
	FImageView(const FImageInfo & InInfo,void * InRawData) : FImageInfo(InInfo), RawData(InRawData)
	{
	}

	// Make an FImageView pointing at an array of FColor pixels as BGRA8 (does not copy the input, points at it)
	FImageView(const FColor * InColors,int32 InSizeX,int32 InSizeY,EGammaSpace InGammaSpace = EGammaSpace::sRGB)
	{
		RawData = (void *) InColors;
		SizeX = InSizeX;
		SizeY = InSizeY;
		NumSlices = 1;
		Format = ERawImageFormat::BGRA8;
		GammaSpace = InGammaSpace; 
	}
	
	// Make an FImageView pointing at an array of FLinearColor pixels as RGBA32F (does not copy the input, points at it)
	FImageView(const FLinearColor * InColors,int32 InSizeX,int32 InSizeY)
	{
		RawData = (void *) InColors;
		SizeX = InSizeX;
		SizeY = InSizeY;
		NumSlices = 1;
		Format = ERawImageFormat::RGBA32F;
		GammaSpace = EGammaSpace::Linear;
	}
	
	// Make an FImageView pointing at an array of FLinearColor pixels as RGBA16F (does not copy the input, points at it)
	FImageView(const FFloat16Color * InColors,int32 InSizeX,int32 InSizeY)
	{
		RawData = (void *) InColors;
		SizeX = InSizeX;
		SizeY = InSizeY;
		NumSlices = 1;
		Format = ERawImageFormat::RGBA16F;
		GammaSpace = EGammaSpace::Linear;
	}

	FImageView(void * InData,int32 InSizeX, int32 InSizeY, int32 InNumSlices, ERawImageFormat::Type InFormat, EGammaSpace InGammaSpace)
		: FImageInfo(InSizeX,InSizeY,InNumSlices,InFormat,InGammaSpace),
		RawData(InData)
	{
	}
	
	FImageView(void * InData,int32 InSizeX, int32 InSizeY, ERawImageFormat::Type InFormat)
		: FImageInfo(InSizeX,InSizeY,1,InFormat, ERawImageFormat::GetDefaultGammaSpace(InFormat) ),
		RawData(InData)
	{
	}

	// get an FImageView to one 2d slice
	IMAGECORE_API FImageView GetSlice(int32 SliceIndex) const;
	
	/**
	 * Copies the image to a destination image with the specified format.
	 *
	 * @param DestImage - The destination image.
	 * @param DestFormat - The destination image format.
	 * @param DestSRGB - Whether the destination image is in SRGB format.
	 */
	IMAGECORE_API void CopyTo(FImage& DestImage, ERawImageFormat::Type DestFormat, EGammaSpace DestGammaSpace) const;
	
	// CopyTo same format
	void CopyTo(FImage& DestImage) const
	{
		CopyTo(DestImage,Format,GammaSpace);
	}
	
	// get a pointer to a pixel
	inline void * GetPixelPointer(int32 X,int32 Y,int32 Slice=0) const
	{
		uint8 * Ptr = (uint8 *)RawData;
		Ptr += GetPixelOffsetBytes(X,Y,Slice);
		return (void *)Ptr;
	}

	// Get one pixel from the image and return in Linear color
	const FLinearColor GetOnePixelLinear(int32 X,int32 Y,int32 Slice=0) const
	{
		void * Ptr = GetPixelPointer(X,Y,Slice);
		return ERawImageFormat::GetOnePixelLinear(Ptr,Format,GammaSpace);
	}

public:

	// Convenience accessors to raw data
	// these are const member functions because the FImageView object itself is const, but the image it points to may not be
	
	TArrayView64<uint8> AsG8() const
	{
		check(Format == ERawImageFormat::G8);
		return { (uint8 *)RawData, GetNumPixels() };
	}

	TArrayView64<uint16> AsG16() const
	{
		check(Format == ERawImageFormat::G16);
		return { (uint16*)RawData, GetNumPixels() };
	}

	TArrayView64<FColor> AsBGRA8() const
	{
		check(Format == ERawImageFormat::BGRA8);
		return { (struct FColor*)RawData, GetNumPixels() };
	}

	TArrayView64<FColor> AsBGRE8() const
	{
		check(Format == ERawImageFormat::BGRE8);
		return { (struct FColor*)RawData, GetNumPixels() };
	}

	TArrayView64<uint16> AsRGBA16() const
	{
		check(Format == ERawImageFormat::RGBA16);
		return { (uint16*)RawData, GetNumPixels() * 4 };
	}

	TArrayView64<FFloat16Color> AsRGBA16F() const
	{
		check(Format == ERawImageFormat::RGBA16F);
		return { (class FFloat16Color*)RawData, GetNumPixels() };
	}

	TArrayView64<FLinearColor> AsRGBA32F() const
	{
		check(Format == ERawImageFormat::RGBA32F);
		check(GammaSpace == EGammaSpace::Linear);
		return { (struct FLinearColor*)RawData, GetNumPixels() };
	}

	TArrayView64<FFloat16> AsR16F() const
	{
		check(Format == ERawImageFormat::R16F);
		return { (class FFloat16*)RawData, GetNumPixels() };
	}
	
	TArrayView64<float> AsR32F() const
	{
		check(Format == ERawImageFormat::R32F);
		return { (float *)RawData, GetNumPixels() };
	}
};

/**
 * Structure for raw image data.
 */
struct FImage : public FImageInfo
{
	/** Raw image data. */
	TArray64<uint8> RawData;

public:

	/**
	 * Default constructor.
	 */
	FImage()
	{
	}
	
	FImage(const FImage & CopyFrom) = default;
	FImage& operator=(const FImage & CopyFrom) = default;

	// move constructor
	inline FImage(FImage && MoveFrom) : FImageInfo(MoveFrom), RawData(MoveTemp(MoveFrom.RawData))
	{
		MoveFrom.Reset();
	}
	
	// move assignment
	FImage& operator=(FImage && MoveFrom)
	{
		if (this != &MoveFrom)
		{
			Swap(MoveFrom);
			MoveFrom.Reset();
		}
		return *this;
	}

	/**
	 * Creates and initializes a new image with the specified number of slices.
	 *
	 * @param InSizeX - The image width.
	 * @param InSizeY - The image height.
	 * @param InNumSlices - The number of slices.
	 * @param InFormat - The image format.
	 * @param InGammaSpace - (optional) Image Gamma Space.  GetDefaultGammaSpace() if not provided.
	 */
	IMAGECORE_API FImage(int32 InSizeX, int32 InSizeY, int32 InNumSlices, ERawImageFormat::Type InFormat, EGammaSpace InGammaSpace);

	// note: changed default GammaSpace from Linear to GetDefaultGammaSpace
	inline FImage(int32 InSizeX, int32 InSizeY, int32 InNumSlices, ERawImageFormat::Type InFormat) :
		FImage(InSizeX,InSizeY,InNumSlices,InFormat,ERawImageFormat::GetDefaultGammaSpace(InFormat))
	{
	}

	/**
	 * Creates and initializes a new image with a single slice.
	 *
	 * @param InSizeX - The image width.
	 * @param InSizeY - The image height.
	 * @param InFormat - The image format.
	 * @param InGammaSpace - (optional) Image Gamma Space.  GetDefaultGammaSpace() if not provided.
	 */
	inline FImage(int32 InSizeX, int32 InSizeY, ERawImageFormat::Type InFormat, EGammaSpace InGammaSpace) :
		FImage(InSizeX,InSizeY,1,InFormat,InGammaSpace)
	{
	}

	// note: changed default GammaSpace from Linear to GetDefaultGammaSpace
	inline FImage(int32 InSizeX, int32 InSizeY, ERawImageFormat::Type InFormat) :
		FImage(InSizeX,InSizeY,1,InFormat,ERawImageFormat::GetDefaultGammaSpace(InFormat))
	{
	}


public:

	// Free FImage.RawData
	//	if bAsyncDetached free is done on a Task, not immediately, but the RawData member is empty upon return
	//	note that FreeData does not change the FImageInfo fields, only frees RawData, follow up with Reset() if desired
	IMAGECORE_API void FreeData(bool bAsyncDetached);

	// Swap the contents of this FImage with another
	inline void Swap(FImage & Other)
	{
		::Swap(RawData,Other.RawData);
		// FImageInfo part be swapped with assignment
		::Swap<FImageInfo>(*this,Other); // Templates/UnrealTemplate.h
	}
	
	// Reset this FImage to empty default constructed state
	inline void Reset()
	{
		FImage Nada;
		Swap(Nada);
	}

	// implicit conversion to FImageView
	// allows FImage to be passed to functions that take FImageView
	// functions that read images should use "const FImageView &" as their argument type
	inline operator FImageView () const
	{
		// copy the shared FImageInfo part :
		const FImageInfo & Info = *this;
		return FImageView( Info , (void *) RawData.GetData() );
	}

	// get an FImageView to one 2d slice
	IMAGECORE_API FImageView GetSlice(int32 SliceIndex) const;
	
	/**
	 * Copies the image to a destination image with the specified format.
	 *
	 * @param DestImage - The destination image.  Will be allocated.  Any existing contents are replaced.
	 * @param DestFormat - The destination image format.
	 * @param DestSRGB - Whether the destination image is in SRGB format.
	 */
	IMAGECORE_API void CopyTo(FImage& DestImage, ERawImageFormat::Type DestFormat, EGammaSpace DestGammaSpace) const;
	
	// CopyTo same format
	void CopyTo(FImage& DestImage) const
	{
		CopyTo(DestImage,Format,GammaSpace);
	}
	
	// in-place format change :
	// does nothing if already in the desired format
	IMAGECORE_API void ChangeFormat(ERawImageFormat::Type DestFormat, EGammaSpace DestGammaSpace);

	/**
	 * Copies and resizes the image to a destination image with the specified size and format.
	 * Resize is done using bilinear filtering; gamma correct (light linear) by converting through RGBA32F.
	 *
	 * legacy API, prefer ResizeImage
	 *
	 * @param DestImage - The destination image.
	 * @param DestSizeX - Width of the resized image
	 * @param DestSizeY - Height of the resized image
	 * @param DestFormat - The destination image format.
	 * @param DestSRGB - Whether the destination image is in SRGB format.
	 */
	IMAGECORE_API void ResizeTo(FImage& DestImage, int32 DestSizeX, int32 DestSizeY, ERawImageFormat::Type DestFormat, EGammaSpace DestGammaSpace) const;

	/**
	 * Linearize to a RGBA32F destination image by applying the decoding function that corresponds to the specified source encoding.
	 * If None, this call will be equivalent to CopyTo(DestImage, ERawImageFormat::RGBA32F, EGammaSpace::Linear).
	 *
	 * @param SourceEncoding - Opaque source encoding (matching UE::Color::EEncoding).
	 * @param DestImage - The destination image.
	 * 
	 * SourceEncoding = None (0) to Linearize using the GammaSpace in this FImage
	 */
	IMAGECORE_API void Linearize(uint8 SourceEncoding, FImage& DestImage) const;

	inline void Linearize(FImage& DestImage) const
	{
		Linearize(0,DestImage);
	}

	UE_DEPRECATED(5.3, "TransformToWorkingColorSpace is deprecated, please use the function in FImageCore.")
	IMAGECORE_API void TransformToWorkingColorSpace(const FVector2d& SourceRedChromaticity, const FVector2d& SourceGreenChromaticity, const FVector2d& SourceBlueChromaticity, const FVector2d& SourceWhiteChromaticity, UE::Color::EChromaticAdaptationMethod Method, double EqualityTolerance = 1.e-7);

	/**
	 * Initializes this image with the specified number of slices.
	 * Allocates RawData.
	 *
	 * @param InSizeX - The image width.
	 * @param InSizeY - The image height.
	 * @param InNumSlices - The number of slices.
	 * @param InFormat - The image format.
	 * @param bInSRGB - Whether the color values are in SRGB format.
	 */
	IMAGECORE_API void Init(int32 InSizeX, int32 InSizeY, int32 InNumSlices, ERawImageFormat::Type InFormat, EGammaSpace InGammaSpace);
	
	inline void Init(int32 InSizeX, int32 InSizeY, int32 InNumSlices, ERawImageFormat::Type InFormat)
	{
		Init(InSizeX,InSizeY,InNumSlices,InFormat,ERawImageFormat::GetDefaultGammaSpace(InFormat));
	}

	/**
	 * Initializes this image with a single slice.
	 * Allocates RawData.
	 *
	 * @param InSizeX - The image width.
	 * @param InSizeY - The image height.
	 * @param InFormat - The image format.
	 * @param bInSRGB - Whether the color values are in SRGB format.
	 */
	IMAGECORE_API void Init(int32 InSizeX, int32 InSizeY, ERawImageFormat::Type InFormat, EGammaSpace InGammaSpace);
	
	inline void Init(int32 InSizeX, int32 InSizeY, ERawImageFormat::Type InFormat)
	{
		Init(InSizeX,InSizeY,InFormat,ERawImageFormat::GetDefaultGammaSpace(InFormat));
	}

	/**
	 * Initializes this image with specified Info.
	 * Allocates RawData.
	 *
	 * @param Info - The image description.  Can also pass an FImage or FImageView to this function.
	 */
	IMAGECORE_API void Init(const FImageInfo & Info);
	
	// get a pointer to a pixel
	inline void * GetPixelPointer(int32 X,int32 Y,int32 Slice=0) const
	{
		int64 Offset = GetPixelOffsetBytes(X,Y,Slice);
		return (void *)&RawData[Offset];
	}

	// Get one pixel from the image and return in Linear color
	const FLinearColor GetOnePixelLinear(int32 X,int32 Y,int32 Slice=0) const
	{
		void * Ptr = GetPixelPointer(X,Y,Slice);
		return ERawImageFormat::GetOnePixelLinear(Ptr,Format,GammaSpace);
	}

public:

	// Convenience accessors to raw data
	
	TArrayView64<uint8> AsG8()
	{
		check(Format == ERawImageFormat::G8);
		return { RawData.GetData(), int64(RawData.Num()) };
	}

	TArrayView64<uint16> AsG16()
	{
		check(Format == ERawImageFormat::G16);
		return { (uint16*)RawData.GetData(), int64(RawData.Num() / sizeof(uint16)) };
	}

	TArrayView64<FColor> AsBGRA8()
	{
		check(Format == ERawImageFormat::BGRA8);
		return { (struct FColor*)RawData.GetData(), int64(RawData.Num() / sizeof(FColor)) };
	}

	TArrayView64<FColor> AsBGRE8()
	{
		check(Format == ERawImageFormat::BGRE8);
		return { (struct FColor*)RawData.GetData(), int64(RawData.Num() / sizeof(FColor)) };
	}

	TArrayView64<uint16> AsRGBA16()
	{
		check(Format == ERawImageFormat::RGBA16);
		return { (uint16*)RawData.GetData(), int64(RawData.Num() / sizeof(uint16)) };
	}

	TArrayView64<FFloat16Color> AsRGBA16F()
	{
		check(Format == ERawImageFormat::RGBA16F);
		return { (class FFloat16Color*)RawData.GetData(), int64(RawData.Num() / sizeof(FFloat16Color)) };
	}

	TArrayView64<FLinearColor> AsRGBA32F()
	{
		check(Format == ERawImageFormat::RGBA32F);
		return { (struct FLinearColor*)RawData.GetData(), int64(RawData.Num() / sizeof(FLinearColor)) };
	}

	TArrayView64<FFloat16> AsR16F()
	{
		check(Format == ERawImageFormat::R16F);
		return { (class FFloat16*)RawData.GetData(), int64(RawData.Num() / sizeof(FFloat16)) };
	}
	
	TArrayView64<float> AsR32F()
	{
		check(Format == ERawImageFormat::R32F);
		return { (float *)RawData.GetData(), int64(RawData.Num() / sizeof(float)) };
	}

	// Convenience accessors to const raw data

	TArrayView64<const uint8> AsG8() const
	{
		check(Format == ERawImageFormat::G8);
		return { RawData.GetData(), int64(RawData.Num()) };
	}

	TArrayView64<const uint16> AsG16() const
	{
		check(Format == ERawImageFormat::G16);
		return { (const uint16*)RawData.GetData(), int64(RawData.Num() / sizeof(uint16)) };
	}

	TArrayView64<const FColor> AsBGRA8() const
	{
		check(Format == ERawImageFormat::BGRA8);
		return { (const struct FColor*)RawData.GetData(), int64(RawData.Num() / sizeof(FColor)) };
	}

	TArrayView64<const FColor> AsBGRE8() const
	{
		check(Format == ERawImageFormat::BGRE8);
		return { (struct FColor*)RawData.GetData(), int64(RawData.Num() / sizeof(FColor)) };
	}

	TArrayView64<const uint16> AsRGBA16() const
	{
		check(Format == ERawImageFormat::RGBA16);
		return { (const uint16*)RawData.GetData(), int64(RawData.Num() / sizeof(uint16)) };
	}

	TArrayView64<const FFloat16Color> AsRGBA16F() const
	{
		check(Format == ERawImageFormat::RGBA16F);
		return { (const class FFloat16Color*)RawData.GetData(), int64(RawData.Num() / sizeof(FFloat16Color)) };
	}

	TArrayView64<const FLinearColor> AsRGBA32F() const
	{
		check(Format == ERawImageFormat::RGBA32F);
		return { (struct FLinearColor*)RawData.GetData(), int64(RawData.Num() / sizeof(FLinearColor)) };
	}

	TArrayView64<const FFloat16> AsR16F() const
	{
		check(Format == ERawImageFormat::R16F);
		return { (const class FFloat16*)RawData.GetData(), int64(RawData.Num() / sizeof(FFloat16)) };
	}
	
	TArrayView64<const float> AsR32F() const
	{
		check(Format == ERawImageFormat::R32F);
		return { (const float*)RawData.GetData(), int64(RawData.Num() / sizeof(float)) };
	}
};

typedef TRefCountPtr<struct FSharedImage> FSharedImageRef;
typedef TRefCountPtr<const struct FSharedImage> FSharedImageConstRef;
struct FSharedImage : public FImage, public FRefCountedObject
{
	FSharedImage() = default;
	virtual ~FSharedImage() = default;
};

/**
 * Structure for raw image data used for Image formats that support Mip Maps.
 */
struct FMipMapImage
{
	struct FMipInfo
	{
		/** Size of the mip image in Pixels*/
		int32 Width = 0;

		/** Size of the mip image in Pixels*/
		int32 Height = 0;

		/** Offset into RawData */
		int64 Offset = 0;

		/** Size of the mip image in Bytes*/
		int64 Size = 0;
	};

	/** Raw image data. */
	TArray64<uint8> RawData;

	/** Holds information about the sub images some image formats support. */
	TArray<FMipInfo> SubImages;

	/** Format in which the images is stored. */
	ERawImageFormat::Type Format = ERawImageFormat::BGRA8;

	/** The gamma space the image is stored in. */
	EGammaSpace GammaSpace = EGammaSpace::sRGB;

public:
	/**
	 * Initializes the Mip Map Image with parameters with uninitialized SubImages.
	 * 
	 * @param InFormat Raw Image Format
	 * @param InGammaSpace Gamma Space
	 */
	IMAGECORE_API void Init(ERawImageFormat::Type InFormat, EGammaSpace InGammaSpace);

	/**
	 * Initializes the Mip Map Image with reserving space for mip maps.
	 *
	 * @param InFormat Raw Image Format
	 * @param InGammaSpace Gamma Space
	 */
	IMAGECORE_API void Init(int32 MipZeroWidth, int32 MipZeroHeight, int32 NumMips, ERawImageFormat::Type InFormat, EGammaSpace InGammaSpace);

	/**
	 * Copies the image to a destination image with the specified format.
	 *
	 * @param DestImage - The destination image.  Will be allocated.  Any existing contents are replaced.
	 * @param DestFormat - The destination image format.
	 * @param DestSRGB - Whether the destination image is in SRGB format.
	 */
	IMAGECORE_API void CopyTo(FMipMapImage& DestImage, ERawImageFormat::Type DestFormat, EGammaSpace DestGammaSpace);

	void CopyTo(FMipMapImage& DestImage)
	{
		CopyTo(DestImage, Format, GammaSpace);
	}

	/**
	 * In - place format change
	 * does nothing if already in the desired format
	 */ 
	IMAGECORE_API void ChangeFormat(ERawImageFormat::Type DestFormat, EGammaSpace DestGammaSpace);


	inline bool IsValid() const
	{
		bool bMipInfoValid = true;
		for (const FMipInfo& MipInfo : SubImages)
		{
			bMipInfoValid &= (MipInfo.Width > 0 && MipInfo.Height > 0 && MipInfo.Size > 0);
		}

		if (!bMipInfoValid) return false;
		if (Format == ERawImageFormat::Invalid) return false;
		if (GammaSpace == EGammaSpace::Invalid) return false;
		if (!GetFormatNeedsGammaSpace(Format) && GammaSpace != EGammaSpace::Linear) return false;
		return true;
	}

	inline bool GetMipDimensions(int32 MipLevel, int32& OutWidth, int32& OutHeight) const
	{
		if (!SubImages.IsValidIndex(MipLevel))
		{
			OutWidth = 0;
			OutHeight = 0;
			return false;
		}

		OutWidth = SubImages[MipLevel].Width;
		OutHeight = SubImages[MipLevel].Height;
		return true;
	}

	inline int32 GetMipCount() const 
	{
		return SubImages.Num();
	}

	FImageView GetMipImage(int32 MipLevel)
	{
		check(SubImages.IsValidIndex(MipLevel));
		return FImageView(static_cast<void*>(RawData.GetData() + SubImages[MipLevel].Offset), SubImages[MipLevel].Width, SubImages[MipLevel].Height, 1, Format, GammaSpace);
	}

	void AddMipImage(TArray64<uint8>&& Buffer, int32 Width, int32 Height)
	{
		FMipInfo MipInfo;
		{
			MipInfo.Width = Width;
			MipInfo.Height = Height;
			MipInfo.Offset = RawData.Num();
			MipInfo.Size = Width * Height * ERawImageFormat::GetBytesPerPixel(Format);
		}
		SubImages.Add(MipInfo);
		RawData.Append(MoveTemp(Buffer));
	}

	inline int64 GetNumPixels()
	{
		return (RawData.Num() / ERawImageFormat::GetBytesPerPixel(Format));
	}

};


/* Functions
 *****************************************************************************/

IMAGECORE_API int32 ImageParallelForComputeNumJobsForPixels(int64 & OutNumPixelsPerJob,int64 NumPixels);

IMAGECORE_API int32 ImageParallelForComputeNumJobsForRows(int32 & OutNumItemsPerJob,int64 SizeX,int64 SizeY);

namespace FImageCore
{

/**
 * Copy FImageView data, converting pixels if necessary.
 * Sizes must match. Dest must already be allocated.
 * For a function that allocates Dest, use CopyTo().
 * 
 * Because FImage implicitly convers to FImageView, these functions can be used on FImage types as well.
 *
 * @param SrcImage - The source image to copy from.
 * @param DestImage - The destination image to copy to. (the FImageView is const but what it points at is not)
 */
IMAGECORE_API void CopyImage(const FImageView & SrcImage,const FImageView & DestImage);

/**
 * Copy Image, swapping RB, SrcImage must be BGRA8 or RGBA16.
 * Sizes must match. Dest must already be allocated.
 * Src == Dest is okay for in-place transpose.
 *
 * @param SrcImage - The source image to copy from.
 * @param DestImage - The destination image to copy to. (the FImageView is const but what it points at is not)
 */
IMAGECORE_API void CopyImageRGBABGRA(const FImageView & SrcImage,const FImageView & DestImage);

/**
 * Copy Image to 2xU16 ; Dest Image should be allocated as BGRA8 but is actually 2xU16
 * Sizes must match. Dest must already be allocated.
 *
 * @param SrcImage - The source image to copy from.
 * @param DestImage - The destination image to copy to. (the FImageView is const but what it points at is not)
 */
IMAGECORE_API void CopyImageTo2U16(const FImageView & SrcImage,const FImageView & DestImage);

/**
* Swap RB channels on Image.  Image must be BGRA8 or RGBA16.
* 
* @param  Image is modified (the FImageView is const but what it points at is not)
*/
IMAGECORE_API void TransposeImageRGBABGRA(const FImageView & Image);


/**
* Clamp Float16 colors which aren't encodable in the BC6H format
* RGB is clamped to non-negative and finite
* A is set to 1.0
*
* @param  InOutImage is modified.  Must be RGBA16F. (the FImageView is const but what it points at is not)
*/
IMAGECORE_API void SanitizeFloat16AndSetAlphaOpaqueForBC6H(const FImageView & InOutImage);


/**
 * Detects whether or not the image contains an alpha channel where at least one texel is != 255.
 * 
 * @param InImage - The image to look for alpha in.
 * @return True if the image both supports and uses the alpha channel. False otherwise.
 */
IMAGECORE_API bool DetectAlphaChannel(const FImageView & InImage);

/**
 * Change alpha channel to opaque, if present
 * 
 * @param InImage - The image to look for alpha in.
 */
IMAGECORE_API void SetAlphaOpaque(const FImageView & InImage);

/**
	* Copies and resizes the image to a destination image with the specified size and format.
	* Resize is done using bilinear filtering; gamma correct (light linear) by converting through RGBA32F.
	*
	* legacy API, prefer ResizeImage
	*
	* @param DestImage - The destination image.
	* @param DestSizeX - Width of the resized image
	* @param DestSizeY - Height of the resized image
	* @param DestFormat - The destination image format.
	* @param DestSRGB - Whether the destination image is in SRGB format.
	*/
IMAGECORE_API void ResizeTo(const FImageView & SourceImage,FImage& DestImage, int32 DestSizeX, int32 DestSizeY, ERawImageFormat::Type DestFormat, EGammaSpace DestGammaSpace);

/**
 * Compute the min/max of each channel to get value ranges
 * Colors are converted to float Linear Gamma
 * 
 * @param InImage - The image to scan
 * @param OutMin - filled with the minimum of the color channels
 * @param OutMax - filled with the maximum of the color channels
 */
IMAGECORE_API void ComputeChannelLinearMinMax(const FImageView & InImage, FLinearColor & OutMin, FLinearColor & OutMax);

/**
* If the image has any values outside the [0,1] range, rescale that edge of the domain so that it is in [0,1]
* does not affect images that were previously in [0,1]
*
* also does not change the side of the domain that is not out of bounds
* eg. values in [0.25,200.0] will be rescaled to [0.25,1.0]
*
* returns bool if any change was made
*
* If the input format is U8 or U16, no change will ever be made and this will return false.
*
* This can be useful if you want to save an HDR/float image to a U8 image format for visualization.
* This is equivalent to what's called the "UNorm" transformation by the RenderTarget ReadPixels functions.
*/
IMAGECORE_API bool ScaleChannelsSoMinMaxIsInZeroToOne(const FImageView & ImageToModify);

/** ComputeImageLinearAverage
 * compute the average linear color of the image
 *	image can be any pixel format
 *	parallel processing is used, but the result is not machine-dependent
 */
IMAGECORE_API FLinearColor ComputeImageLinearAverage(const FImageView & Image);

/**
 * Apply a color space transformation from the source chromaticities to the engine's working color space.
 *
 * @param InLinearImage - The image to convert, which must be linear.
 * @param SourceRedChromaticity - The red chromaticity coordinate of the source color space.
 * @param SourceGreenChromaticity - The green chromaticity coordinate of the source color space.
 * @param SourceBlueChromaticity - The blue chromaticity coordinate of the source color space.
 * @param SourceWhiteChromaticity - The white chromaticity coordinate of the source color space.
 * @param Method - The chromatic adapation method.
 * @param EqualityTolerance - The tolerance for the source and working color space chromaticities to be considered equal, bypassing the transform.
 */
IMAGECORE_API void TransformToWorkingColorSpace(const FImageView& InLinearImage, const FVector2d& SourceRedChromaticity, const FVector2d& SourceGreenChromaticity, const FVector2d& SourceBlueChromaticity, const FVector2d& SourceWhiteChromaticity, UE::Color::EChromaticAdaptationMethod Method, double EqualityTolerance = 1.e-7);


	/*
	* filter choice for ResizeImage
	*/
	enum class EResizeImageFilter : uint32
	{
		Default = 0, // uses a good default filter; = AdaptiveSharp
		PointSample,
		Box,
		Triangle, 
		Bilinear = Triangle, // synonym
		CubicGaussian, // smooth Mitchell B=1,C=0, B-spline, Gaussian-like
		CubicSharp, // sharp interpolating cubic, Catmull-Rom (has negative lobes) (Mitchell B=0)
		CubicMitchell, // compromise between sharp and smooth cubic, Mitchell-Netrevalli filter with B=1/3, C=1/3 (has negative lobes)
		AdaptiveSharp,  // sharper adaptive filter; uses CubicSharp for upsample and CubicMitchell for downsample, nop for same size
		AdaptiveSmooth,  // smoother adaptive filter; uses CubicMitchell for upsample and CubicGaussian for downsample, nop for same size

		MitchellOneQuarter, // B=1/4 ("Robidoux")
		MitchellOneSixth, // B=1/6
		MitchellNegOneSixth, // B=-1/6 , over-sharpening
		MitchellNegOneThird, // B=-1/3 , over-sharpening
		Lanczos4, // sharp
		Lanczos5, // very sharp, often ringy

		// cubic Mitchells in order of B: (from highest B to lowest; eg. smoothest to sharpest)
		// CubicGaussian  // B = 1
		// CubicMitchell // B = 1/3
		// MitchellOneQuarter // B = 1/4
		// MitchellOneSixth // B = 1/6
		// CubicSharp  // B = 0
		// MitchellNegOneSixth // B = -1/6
		// MitchellNegOneThird // B = -1/3

		WithoutFlagsMask = 63,
		Flag_WrapX = 64,  // default edge mode is clamp; set these to wrap instead
		Flag_WrapY = 128,
		Flag_AlphaWeighted = 256 // weight RGB by A in the filter; that is, convert non-premultiplied RGBA input to premultiplied for filtering, then convert back
	};
	ENUM_CLASS_FLAGS(EResizeImageFilter);

	/* ResizeImage :
	*	DestImage should be already allocated; DestImage will be filled in specified format
	* filter is always computed in floating point, gamma correct linear light, but no intermediate conversion is done
	* note: some filters (the three "Cubic" options) will change the image even if Source.Size == Dest.Size
	*	this function can be used to apply filters on same-size image with the Cubic set
	* the "Adaptive" (and Default) filters automatically change depending on if the resize is an upsample or downsample, and are nops for same size
	*
	* @param SourceImage - Source Image to resize from
	* @param DestImage - Dest Image to resize to; specifies output size and format.  Note the FImageView itself is const but the Dest pixels are written.  Dest should be already allocated.  Source == Dest in-place resizing not allowed.
	* @param Filter - EResizeImageFilter filter choice, or Default if none
	*/
	IMAGECORE_API void ResizeImage(const FImageView & SourceImage,const FImageView & DestImage, EResizeImageFilter Filter = EResizeImageFilter::Default); 

	/* ResizeImage variant.  See main ResizeImage function for notes.
	*
	* Allocate DestImage (if needed) to specified size and Resize from SourceImage into it
	*	do not use this with Source == Dest (call ResizeImageInPlace)
	*
	* @param SourceImage - Source Image to resize from
	* @param DestImage - Dest to allocated and write to.  Will not be allocated if it is already the right size.  DestImage size/format is replaced.
	* @param DestSizeX - Specifies output of resize.  DestImage will be changed to this.
	* @param DestSizeY - Specifies output of resize.  DestImage will be changed to this.
	* @param DestFormat - Specifies output of resize.  DestImage will be changed to this.
	* @param DestGammaSpace - Specifies output of resize.  DestImage will be changed to this.
	* @param Filter - EResizeImageFilter filter choice, or Default if none
	*/
	IMAGECORE_API void ResizeImageAllocDest(const FImageView & SourceImage,FImage & DestImage,int32 DestSizeX, int32 DestSizeY, ERawImageFormat::Type DestFormat, EGammaSpace DestGammaSpace, EResizeImageFilter Filter = EResizeImageFilter::Default); 
	
	/* ResizeImage variant.  See main ResizeImage function for notes.
	*
	* Allocate DestImage (if needed) to specified size and Resize from SourceImage into it
	*	do not use this with Source == Dest (call ResizeImageInPlace)
	*
	*	DestImage will have same format as Source
	*
	* @param SourceImage - Source Image to resize from
	* @param DestImage - Dest to allocated and write to.  Will not be allocated if it is already the right size.  DestImage size/format is replaced.
	* @param DestSizeX - Specifies output of resize.  DestImage will be changed to this.
	* @param DestSizeY - Specifies output of resize.  DestImage will be changed to this.
	* @param Filter - EResizeImageFilter filter choice, or Default if none
	*/
	IMAGECORE_API void ResizeImageAllocDest(const FImageView & SourceImage,FImage & DestImage,int32 DestSizeX, int32 DestSizeY, EResizeImageFilter Filter = EResizeImageFilter::Default); 

	/* ResizeImage variant.  See main ResizeImage function for notes.
	*
	* Resize from Image and write the result into Image.
	* Image may be reallocated to fit the destination size/format requested.
	*
	* Other ResizeImage functions do not work in-place.
	*
	* @param Image - Source Image to resize from, will be changed to hold the destination image and reallocated if needed.
	* @param DestSizeX - Specifies output of resize.  DestImage will be changed to this.
	* @param DestSizeY - Specifies output of resize.  DestImage will be changed to this.
	* @param DestFormat - Specifies output of resize.  DestImage will be changed to this.
	* @param DestGammaSpace - Specifies output of resize.  DestImage will be changed to this.
	* @param Filter - EResizeImageFilter filter choice, or Default if none
	*/
	IMAGECORE_API void ResizeImageInPlace(FImage & Image,int32 DestSizeX, int32 DestSizeY, ERawImageFormat::Type DestFormat, EGammaSpace DestGammaSpace, EResizeImageFilter Filter = EResizeImageFilter::Default);
	
	/* ResizeImage variant.  See main ResizeImage function for notes.
	*
	* Resize from Image and write the result into Image.
	* Image may be reallocated to fit the destination size/format requested.
	*
	* Other ResizeImage functions do not work in-place.
	*
	* Format of image will not be changed.
	*
	* @param Image - Source Image to resize from, will be changed to hold the destination image and reallocated if needed.
	* @param DestSizeX - Specifies output of resize.  DestImage will be changed to this.
	* @param DestSizeY - Specifies output of resize.  DestImage will be changed to this.
	* @param Filter - EResizeImageFilter filter choice, or Default if none
	*/
	IMAGECORE_API void ResizeImageInPlace(FImage & Image,int32 DestSizeX, int32 DestSizeY, EResizeImageFilter Filter = EResizeImageFilter::Default);
 
	//----------------------

};
