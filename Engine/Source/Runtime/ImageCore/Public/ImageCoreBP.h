// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ImageCore.h"
#include "UObject/Object.h"
#include "ImageCoreBP.generated.h"

#define UE_API IMAGECORE_API

/**
*  Exposes a FSharedImage to blueprint in an unalterable way.
*/
USTRUCT(BlueprintType, meta = (DisplayName = "FSharedImageConst"))
struct FSharedImageConstRefBlueprint
{
	GENERATED_BODY()
	FSharedImageConstRef Reference;

	FSharedImageConstRefBlueprint() = default;
	~FSharedImageConstRefBlueprint() = default;
	FSharedImageConstRefBlueprint(const FSharedImageConstRefBlueprint&) = default;
	FSharedImageConstRefBlueprint(FSharedImageConstRefBlueprint&&) = default;
	FSharedImageConstRefBlueprint& operator=(const FSharedImageConstRefBlueprint&) = default;
	FSharedImageConstRefBlueprint& operator=(FSharedImageConstRefBlueprint&&) = default;
};

UCLASS(MinimalAPI, BlueprintType)
class USharedImageConstRefBlueprintFns : public UObject
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "SharedImage")
    static UE_API bool IsValid(const FSharedImageConstRefBlueprint& Image);

	// Returns (-1, -1) if Image is invalid
	UFUNCTION(BlueprintCallable, Category = "SharedImage")
    static UE_API FVector2f GetSize(const FSharedImageConstRefBlueprint& Image);

	// Returns -1 if Image is invalid
	UFUNCTION(BlueprintCallable, Category = "SharedImage")
    static UE_API int32 GetWidth(const FSharedImageConstRefBlueprint& Image);

	// Returns -1 if Image is invalid
	UFUNCTION(BlueprintCallable, Category = "SharedImage")
    static UE_API int32 GetHeight(const FSharedImageConstRefBlueprint& Image);

	/**
	*	Returns the color value for the given pixel. If the input position is invalid, the format is invalid,
	*	or the reference isn't set, bValid will be false and the function will return FailureColor. The color
	*	is converted using the image's gamma space in to linear space.
	* 
	*	Do not use this for full image processing as it will be extremely slow, contact support if you need such
	*	functionality.
	*/
	UFUNCTION(BlueprintCallable, Category = "SharedImage")
    static UE_API FLinearColor GetPixelLinearColor(const FSharedImageConstRefBlueprint& Image, int32 X, int32 Y, bool& bValid, FLinearColor FailureColor=FLinearColor::Black);


	/**
	*	Returns the value in the texture for the given pixel as a float vector. If the input position is invalid, the format is invalid,
	*	or the reference isn't set, bValid will be false and the function will return FVector4(0,0,0,0).
	* 
	*	Pixel values are directly returned with no gamma transformation to allow for lookup tables. Also note that
	*	8 bit formats that you might normally expect to be normalized to 0..1 will return their values directly as 0..256.
	*
	*	This supports all image formats. 
	* 
	*	G8 is replicated to X/Y/Z/1.
	*	R16/R32 is returned as R/0/0/1.
	* 
	*	Do not use this for full image processing as it will be extremely slow, contact support if you need such
	*	functionality.
	*/
	UFUNCTION(BlueprintCallable, Category = "SharedImage")
    static UE_API FVector4f GetPixelValue(const FSharedImageConstRefBlueprint& Image, int32 X, int32 Y, bool& bValid);
};

#undef UE_API
