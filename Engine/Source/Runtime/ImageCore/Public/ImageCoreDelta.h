// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
ImageCoreUtils.h: Image utility functions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Engine/TextureDefines.h"
#include "ImageCore.h"



// FImageViewStrided cannot derive from FImageView
// you can implicitly convert FImage -> FImageView -> FImageViewStrided
//	but you cannot convert FImageViewStrided -> the others, because they can't have stride
struct FImageViewStrided : public FImageInfo
{
	void * RawData = nullptr;
	int64 StrideBytes = 0;
		
	FImageViewStrided() { }
	
	FImageViewStrided(const FImageInfo & InInfo,void * InRawData,int64 InStride = 0) : FImageInfo(InInfo), RawData(InRawData), StrideBytes(InStride)
	{
		if ( StrideBytes == 0 )
		{
			StrideBytes = InInfo.GetStrideBytes();
		}
	}
	
	FImageViewStrided(const FImageView & InView)
	{
		*((FImageInfo *)this) = InView; // copy the FImageInfo part
		RawData = InView.RawData;
		StrideBytes = InView.GetStrideBytes();
	}

	// replaces the call in FImageInfo but is NOT virtual ; beware
	inline int64 GetStrideBytes()  const { return StrideBytes; }
	
	// if IsStrideWidth, then you can convert back to FImageView
	inline bool IsStrideWidth() const { return StrideBytes == SizeX * GetBytesPerPixel(); }

	// get offset of a pixel from the base pointer, in bytes
	// replaces the call in FImageInfo but is NOT virtual ; beware
	inline int64 GetPixelOffsetBytes(int32 X,int32 Y,int32 Slice = 0) const
	{
		checkSlow( X >= 0 && X < SizeX );
		checkSlow( Y >= 0 && Y < SizeY );
		checkSlow( Slice >= 0 && Slice < NumSlices );

		int64 Offset = Slice * StrideBytes * SizeY;
		Offset += X * GetBytesPerPixel();
		Offset += Y * StrideBytes;

		return Offset;
	}

	// queries like GetImageSizeBytes are ambiguous ; do you mean the used pixels or the stride?

	inline uint8 * GetRowPointer(int32 Y,int32 Slice = 0) const
	{
		checkSlow( Y >= 0 && Y < SizeY );
		checkSlow( Slice >= 0 && Slice < NumSlices );

		int64 Offset = Slice * StrideBytes * SizeY;
		Offset += Y * StrideBytes;

		return (uint8 *)RawData + Offset;
	}
	
	// get a pointer to a pixel
	inline void * GetPixelPointer(int32 X,int32 Y,int32 Slice=0) const
	{
		uint8 * Ptr = (uint8 *)RawData;
		Ptr += GetPixelOffsetBytes(X,Y,Slice);
		return (void *)Ptr;
	}
	
	inline const FImageViewStrided GetPortion(int64 PortionStartX,int64 PortionSizeX, int64 PortionStartY,int64 PortionSizeY) const
	{
		check( PortionStartX >= 0 && (PortionStartX + PortionSizeX) <= SizeX );
		check( PortionStartY >= 0 && (PortionStartY + PortionSizeY) <= SizeY );
		check( NumSlices == 1 ); // does not support slices

		FImageViewStrided Ret = *this;
		Ret.SizeX = PortionSizeX;
		Ret.SizeY = PortionSizeY;
		Ret.RawData = GetPixelPointer(PortionStartX,PortionStartY);
		return Ret;
	}
};


namespace FImageCoreDelta
{
	/**
	Split InView into tiles and add to OutViews.  Will add 1 more entries to OutViews.
	**/
	IMAGECORE_API void AddSplitStridedViewsForDelta( TArray64<FImageViewStrided> & OutViews, const FImageView & InView );
	
	/**
	Do Delta Transform
	bForward=false is the inverse transform

	OutData must be the same size and layout as InImage ; eg. same pixel sizes and strides.  It is not rearranged in any way.
	No headers are added.
	**/
	IMAGECORE_API void DoTransform(const FImageViewStrided & InImage,uint8 * OutData,bool bForward);
};
