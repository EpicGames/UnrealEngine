// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
ImageCoreUtils.h: Image utility functions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Engine/TextureDefines.h"
#include "ImageCore.h"
#include "PixelFormat.h"
#include "UObject/NameTypes.h"

// ImageCore Module does not use Engine, but it can see TextureDefines
// do not use Engine functions here
// 
// keep the ImageCore.h includes as minimal as possible
// put any functions that need more headers here

namespace FImageCoreUtils
{
	// IsImageImportPossible returns false if the image dimensions are not supported to import
	//	if this function returns true, image import might still fail due to further constraints
	//	(this constraint is weaker than the one actually required)
	//  (see IsImportResolutionValid, which is also still weaker than the final constraint)
	// IsImageImportPossible does not check current Engine config or platform limits, only hard-coded absolute limits
	IMAGECORE_API bool IsImageImportPossible(int64 Width,int64 Height);

	// Returns the number of mips that constitute a full mip chain for the given top level mip size.
	// InVolumeZ is ignored unless bInIsVolume is true
	IMAGECORE_API int32 GetMipCountFromDimensions(int32 InSizeX, int32 InSizeY, int32 InVolumeZ, bool bInIsVolume);

	// ETextureSourceFormat and ERawImageFormat::Type are one-to-one :
	IMAGECORE_API ERawImageFormat::Type ConvertToRawImageFormat(ETextureSourceFormat Format);
	IMAGECORE_API ETextureSourceFormat ConvertToTextureSourceFormat(ERawImageFormat::Type Format);

	// EPixelFormat is the graphics texture pixel format
	// it is a much larger superset of ERawImageFormat
	// GetPixelFormatForRawImageFormat does not map to the very closest EPixelFormat
	//	instead map to a close one that is actually usable as Texture
	// if *pOutEquivalentFormat != InFormat , then conversion is needed
	//  and conversion can be done using CopyImage to OutEquivalentFormat 
	IMAGECORE_API EPixelFormat GetPixelFormatForRawImageFormat(ERawImageFormat::Type InFormat, 
							ERawImageFormat::Type * pOutEquivalentFormat=nullptr);

							
	IMAGECORE_API FName ConvertToUncompressedTextureFormatName(ERawImageFormat::Type Format);

	/**
	 * Returns ETextureSourceFormat which can be used to efficiently store data encoded in both input formats.
	 * Can be used in cases when multiple sources need to be mixed together (i.e. in a texture array or UDIM)
	 * @param Format1 - First source format.
	 * @param Format2 - Second source format.
	 */
	IMAGECORE_API ETextureSourceFormat GetCommonSourceFormat(ETextureSourceFormat Format1, ETextureSourceFormat Format2);
	
	// return an ERawImageFormat for a PixelFormat
	// these are not equal and you cannot blit between them
	//	it's just the closest reasonable thing that can hold that PixelFormat
	//	often there is no good match
	IMAGECORE_API ERawImageFormat::Type GetRawImageFormatForPixelFormat(EPixelFormat PF);
};
