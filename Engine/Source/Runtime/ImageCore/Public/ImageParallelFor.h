// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
ImageParallelFor.h: helper to run ParallelFor on images
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "ImageCore.h"
#include "Async/ParallelFor.h"

/**
 * 

 The easy and efficient way to write a pixel processing loop that supports all pixel formats is to use

 ImageParallelProcessLinearPixels
 
 passing in a lambda like :
 ProcessLinearPixelsAction ProcessPixels(TArrayView64<FLinearColor> Colors)

 this will let you visit all the pixels as FLinearColor.

 You can use this for read (return ProcessLinearPixelsAction::ReadOnly) or read-write (return ProcessLinearPixelsAction::Modified).

 */

// ImageCore.h in global namespace:
// ImageParallelForComputeNumJobsForPixels
// ImageParallelForComputeNumJobsForRows

namespace FImageCore
{
	// ImageParallelFor treats the image as being 2d (no slices) but with *NumSlices rows
	inline int64 ImageParallelForComputeNumRows(const FImageView & Image)
	{
		return (int64)Image.SizeY * Image.NumSlices;
	}
	
	// Y indexes across slices and SizeY here :
	inline FImageView ImageParallelForGetOneRowView(const FImageView & Image,int64 Y)
	{
		check( Y >= 0 && Y < ImageParallelForComputeNumRows(Image) );

		// point at one row of the image :
		FImageView ImageRow = Image;
		ImageRow.SizeY = 1;
		ImageRow.NumSlices = 1;
		ImageRow.RawData = (uint8 *)Image.RawData + Image.GetBytesPerPixel() * Y * Image.SizeX;
		return ImageRow;
	}

	IMAGECORE_API int32 ImageParallelForComputeNumJobs(const FImageView & Image,int64 * pRowsPerJob);
	IMAGECORE_API int64 ImageParallelForMakePart(FImageView * Part,const FImageView & Whole,int64 JobIndex,int64 RowsPerJob);

	// Lambda is a functor that works on an FImageView()
	//	it will be called with portions of the image
	//	each portion will be a 2d FImageView (NumSlices == 1)
	// use like :
	// FImageCore::ImageParallelFor( TEXT("Texture.AdjustImageColorsFunc.PF"),Image, [&](FImageView & ImagePart) {
	template <typename Lambda>
	inline void ImageParallelFor(const TCHAR* DebugName, const FImageView & Image, const Lambda& Func)
	{
		int64 RowsPerJob;
		int32 NumJobs = ImageParallelForComputeNumJobs(Image,&RowsPerJob);
		
		ParallelFor(DebugName, NumJobs, 1, [=](int64 JobIndex)
		{
			FImageView Part;
			int64 StartY = ImageParallelForMakePart(&Part,Image,JobIndex,RowsPerJob);
			Func(Part,StartY);
		}, EParallelForFlags::Unbalanced);
	}
	
	enum class ProcessLinearPixelsAction
	{
		ReadOnly,
		Modified
	};

	/**
	 * ProcessLinearPixels
	 *  act on image pixels as TArrayView64<FLinearColor> Colors
	 * pass a lambda that is :
	 *  ProcessLinearPixelsAction ProcessPixels(TArrayView64<FLinearColor> Colors)
	 *    return ProcessLinearPixelsAction::Modified if you modify colors
	 *    return ProcessLinearPixelsAction::ReadOnly if you only read and don't change colors
	 * your lambda will be called on one row at a time
	 */
	template <typename Lambda>
	inline void ProcessLinearPixels(const FImageView & Image, const Lambda& Func, int64 StartY)
	{
		if ( Image.Format == ERawImageFormat::RGBA32F )
		{
			// Image is already FLinearColor, can work on it in-place
			TArrayView64<FLinearColor> Colors = Image.AsRGBA32F();
			FLinearColor * Start = &Colors[0];

			for(int64 Y=0;Y<ImageParallelForComputeNumRows(Image);Y++)
			{
				TArrayView64<FLinearColor> RowColors(Start+Y*Image.SizeX,Image.SizeX);
				Func(RowColors,StartY+Y);
			}
		}
		else
		{
			// convert to linear, act on linear, then copy back
			FImage LinearRow(Image.SizeX,1,1,ERawImageFormat::RGBA32F,EGammaSpace::Linear);
			TArrayView64<FLinearColor> RowColors = LinearRow.AsRGBA32F();
			
			for(int64 Y=0;Y<ImageParallelForComputeNumRows(Image);Y++)
			{
				// point at one row of the image :
				FImageView ImageRow = ImageParallelForGetOneRowView(Image,Y);

				CopyImage(ImageRow,LinearRow);
			
				ProcessLinearPixelsAction Action = Func(RowColors,StartY+Y);

				if ( Action == ProcessLinearPixelsAction::Modified )
				{
					// Colors were modified, need to blit back
					CopyImage(LinearRow,ImageRow);
				}
			}
		}
	}

	// parallel process image as TArrayView64<FLinearColor>
	//	your lambda is called on one row at a time so that the grouping is machine invariant
	template <typename Lambda>
	inline void ImageParallelProcessLinearPixels(const TCHAR* DebugName, const FImageView & Image, const Lambda& Func)
	{
		ImageParallelFor(DebugName,Image,[Func](const FImageView &Part,int64 StartY) {
			// call ProcessLinearPixels on each part :
			ProcessLinearPixels(Part,Func,StartY);
		} );
	}

};
